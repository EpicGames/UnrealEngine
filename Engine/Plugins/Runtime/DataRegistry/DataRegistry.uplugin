{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Data Registry",
	"Description" : "Adds Data Registry system that can be used as a generic interface for acquiring structure data from multiple sources at runtime",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "DataRegistry",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "DataRegistryEditor",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		}
	]
}