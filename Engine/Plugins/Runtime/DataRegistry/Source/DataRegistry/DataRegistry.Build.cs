// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataRegistry : ModuleRules
	{
		public DataRegistry(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"DeveloperSettings"
				}
			);

			// Needed for PIE callbacks, which should really be somewhere better
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd"
					}
				);
			}
		}
	}
}
