// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistry.h"
#include "DataRegistrySource.h"
#include "DataRegistryTypesPrivate.h"
#include "DataRegistrySubsystem.h"
#include "Engine/StreamableManager.h"
#include "TimerManager.h"
#include "Engine/AssetManager.h"
#include "Engine/CurveTable.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "GameplayTagsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistry)

#define LOCTEXT_NAMESPACE "DataRegistryEditor"

UDataRegistry::UDataRegistry()
	: Cache(new FDataRegistryCache())
{

}

UDataRegistry::~UDataRegistry()
{
	
}

void UDataRegistry::BeginDestroy()
{
	if (Cache && !IsEngineExitRequested())
	{
		Cache->ClearCache(true);

		// Can't be a fancy template because UObjects want the all members fully defined, and I want it to be raw for access perf
		delete Cache;
	}

	Super::BeginDestroy();
}

const UScriptStruct* UDataRegistry::GetItemStruct() const
{
	return ItemStruct;
}

bool UDataRegistry::DoesItemStructMatchFilter(FName FilterStructName) const
{
	if (FilterStructName == NAME_None)
	{
		return true;
	}

	UStruct* CheckStruct = (UStruct*)ItemStruct;

	while (CheckStruct)
	{
		if (CheckStruct->GetFName() == FilterStructName)
		{
			return true;
		}
		CheckStruct = CheckStruct->GetSuperStruct();
	}

	return false;
}

const FDataRegistryIdFormat& UDataRegistry::GetIdFormat() const
{
	return IdFormat;
}

const FName UDataRegistry::GetRegistryType() const
{
	return RegistryType;
}

FText UDataRegistry::GetRegistryDescription() const
{
	FText FormatDescription;
	FDataRegistryIdFormat TempIdFormat = GetIdFormat();

	if (TempIdFormat.BaseGameplayTag.IsValid())
	{
		FormatDescription = LOCTEXT("FormatDescription_UsingTags", "using tags");
	}
	else
	{
		FormatDescription = LOCTEXT("FormatDescription_UsingNames", "using names");
	}

	return FText::Format(LOCTEXT("RegistryDescription_Format", "{0}: Type {1}, {2}"), FText::AsCultureInvariant(GetName()), FText::AsCultureInvariant(GetRegistryType().ToString()), FormatDescription);
}

bool UDataRegistry::IsInitialized() const
{
	return bIsInitialized;
}

bool UDataRegistry::Initialize()
{
	bool bAnyInitialized = false;
	for (UDataRegistrySource* Source : DataSources)
	{
		if (Source)
		{
			if (Source->IsInitialized())
			{
				bAnyInitialized = true;
			}
			else
			{
				bAnyInitialized |= Source->Initialize();
			}
		}
	}

	if (bAnyInitialized)
	{
		RefreshRuntimeSources();

		bIsInitialized = true;

		RuntimeCachePolicy = DefaultCachePolicy;

		// Start update timer
		FTimerManager* TimerManager = GetTimerManager();
		check(TimerManager);
		TimerManager->SetTimer(UpdateTimer, this, &UDataRegistry::TimerUpdate, TimerUpdateFrequency, true, FMath::FRand() * TimerUpdateFrequency);
	}

	return bIsInitialized;
}

void UDataRegistry::Deinitialize()
{
	if (bIsInitialized)
	{
		ResetRuntimeState();

		// Deinitialize original sources first
		for (int32 i = 0; i < DataSources.Num(); i++)
		{
			UDataRegistrySource* Source = DataSources[i];
			if (Source)
			{
				Source->Deinitialize();
			}
		}

		FTimerManager* TimerManager = GetTimerManager();
		if (TimerManager)
		{
			TimerManager->ClearTimer(UpdateTimer);
		}
	
		bIsInitialized = false;
	
		// Now refresh runtime sources, this will clear out any leftovers
		RefreshRuntimeSources();

		ensureMsgf(RuntimeSources.Num() == 0, TEXT("Some RuntimeSources did not correctly deinitialize!"));
	}
}

void UDataRegistry::ResetRuntimeState()
{
	RuntimeCachePolicy = DefaultCachePolicy;

	for (UDataRegistrySource* Source : RuntimeSources)
	{
		if (Source)
		{
			Source->ResetRuntimeState();
		}
	}

	Cache->ClearCache(true);
	InvalidateCacheVersion();
}

void UDataRegistry::MarkRuntimeDirty()
{
	bNeedsRuntimeRefresh = true;
}

void UDataRegistry::RuntimeRefreshIfNeeded()
{
	if (bNeedsRuntimeRefresh && bIsInitialized)
	{
		RefreshRuntimeSources();
	}
}

EDataRegistryRegisterAssetResult UDataRegistry::RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority /*= 0*/)
{
	EDataRegistryRegisterAssetResult RegisterAssetResult = EDataRegistryRegisterAssetResult::NotRegistered;

	for (int32 i = 0; i < DataSources.Num(); i++)
	{
		UDataRegistrySource* Source = DataSources[i];
		if (Source)
		{
			EDataRegistryRegisterAssetResult CurrentRegisterAssetResult = Source->RegisterSpecificAsset(AssetData, AssetPriority);

			// Don't clobber Asset already registered if Not Registered, Only move to a higher Result state, with success being the highest.
			if (CurrentRegisterAssetResult > RegisterAssetResult)
			{
				RegisterAssetResult = CurrentRegisterAssetResult;
			}
		}
	}

	if (DidRegisterAssetResultCauseChange(RegisterAssetResult) && IsInitialized())
	{
		// Don't want to do a full reset, but do clear cache as lookup rules may have changed
		RefreshRuntimeSources();
	}
	
	return RegisterAssetResult;
}

bool UDataRegistry::DidRegisterAssetResultCauseChange(EDataRegistryRegisterAssetResult RegisterAssetStatus) const
{
	return RegisterAssetStatus == EDataRegistryRegisterAssetResult::RegisteredSuccesfully;
}

bool UDataRegistry::UnregisterSpecificAsset(const FSoftObjectPath& AssetPath)
{
	bool bMadeChange = false;

	for (int32 i = 0; i < DataSources.Num(); i++)
	{
		UDataRegistrySource* Source = DataSources[i];
		if (Source && Source->UnregisterSpecificAsset(AssetPath))
		{
			bMadeChange = true;
		}
	}

	if (bMadeChange && IsInitialized())
	{
		// Don't want to do a full reset, but do clear cache as lookup rules may have changed
		RefreshRuntimeSources();
	}

	return bMadeChange;
}

int32 UDataRegistry::UnregisterAssetsWithPriority(int32 AssetPriority)
{
	int32 NumberUnregistered = 0;

	for (int32 i = 0; i < DataSources.Num(); i++)
	{
		UDataRegistrySource* Source = DataSources[i];
		if (Source)
		{
			NumberUnregistered += Source->UnregisterAssetsWithPriority(AssetPriority);
		}
	}

	if (NumberUnregistered > 0 && IsInitialized())
	{
		// Don't want to do a full reset, but do clear cache as lookup rules may have changed
		RefreshRuntimeSources();
	}

	return NumberUnregistered;
}

EDataRegistryAvailability UDataRegistry::GetLowestAvailability() const
{
	if (RuntimeSources.Num() == 0)
	{
		return EDataRegistryAvailability::DoesNotExist;
	}

	EDataRegistryAvailability LowestAvailability = RuntimeSources[0]->GetSourceAvailability();

	for (int32 i = 1; i < RuntimeSources.Num(); i++)
	{
		EDataRegistryAvailability SourceAvailability = RuntimeSources[i]->GetSourceAvailability();

		if ((uint8)SourceAvailability < (uint8)LowestAvailability)
		{
			LowestAvailability = SourceAvailability;
		}
	}

	return LowestAvailability;
}

const FDataRegistryCachePolicy& UDataRegistry::GetRuntimeCachePolicy() const
{
	if (!bIsInitialized)
	{
		return DefaultCachePolicy;
	}

	return RuntimeCachePolicy;
}

void UDataRegistry::SetRuntimeCachePolicy(const FDataRegistryCachePolicy& NewPolicy)
{
	RuntimeCachePolicy = NewPolicy;
	ApplyCachePolicy();
}

void UDataRegistry::ApplyCachePolicy()
{
	bool bMadeChange = false;
	float CurrentTime = GetCurrentTime();

	TArray<TPair<FCachedDataRegistryItem*, float> > SortedItems;
	TArray<FCachedDataRegistryItem*> ItemsToDelete;

	for (TPair<FDataRegistryLookup, TUniquePtr<FCachedDataRegistryItem>>& Pair : Cache->LookupCache)
	{
		float Relevancy = Pair.Value->GetRelevancy(RuntimeCachePolicy, CurrentTime);
		// Relevancy of 1 will always stay loaded
		if (Relevancy < 1.0f)
		{
			SortedItems.Emplace(Pair.Value.Get(), Relevancy);
		}
	}

	SortedItems.Sort([&](const TPair<FCachedDataRegistryItem*, float>& A, const TPair<FCachedDataRegistryItem*, float>& B) {
		return A.Value < B.Value;
		});

	for (int32 i = 0; i < SortedItems.Num(); i++)
	{
		float Relevancy = SortedItems[i].Value;
		FCachedDataRegistryItem* Item = SortedItems[i].Key;

		int32 CurrentCount = Cache->LookupCache.Num();

		if (Relevancy < 0 && CurrentCount > RuntimeCachePolicy.MinNumberKept)
		{
			// Always delete
			Cache->RemoveCacheEntry(Item->LookupKey);
			bMadeChange = true;
			continue;
		}
		else if (RuntimeCachePolicy.MaxNumberKept && CurrentCount > RuntimeCachePolicy.MaxNumberKept)
		{
			// Delete in priority order
			Cache->RemoveCacheEntry(Item->LookupKey);
			bMadeChange = true;
			continue;
		}
	}

	if (bMadeChange)
	{
		InvalidateCacheVersion();
	}
}

bool UDataRegistry::IsCacheGetResultValid(FDataRegistryCacheGetResult Result) const
{
	if (!Result.IsPersistent())
	{
		return false;
	}

	FDataRegistryCacheGetResult CurrentVersion = GetCacheResultVersion();

	if (Result.IsValidForVersion(CurrentVersion.GetVersionSource(), CurrentVersion.GetCacheVersion()))
	{
		return true;
	}

	return false;
}

FDataRegistryCacheGetResult UDataRegistry::GetCacheResultVersion() const
{
	bool bIsVolatile = RuntimeCachePolicy.bCacheIsAlwaysVolatile || FDataRegistryResolverScope::IsStackVolatile();
	
	if (bIsVolatile)
	{
		// If volatile, version is not returned
		return FDataRegistryCacheGetResult(EDataRegistryCacheGetStatus::FoundVolatile);
	}

	int32 CacheVersion;
	EDataRegistryCacheVersionSource VersionSource;

	if (RuntimeCachePolicy.bUseCurveTableCacheVersion)
	{
		CacheVersion = UCurveTable::GetGlobalCachedCurveID();
		VersionSource = EDataRegistryCacheVersionSource::CurveTable;
	}
	else
	{
		CacheVersion = Cache->CurrentCacheVersion;
		VersionSource = EDataRegistryCacheVersionSource::DataRegistry;
	}

	return FDataRegistryCacheGetResult(EDataRegistryCacheGetStatus::FoundPersistent, VersionSource, CacheVersion);
}

void UDataRegistry::InvalidateCacheVersion()
{
	// TODO Do we need to defer calls to avoid multiple callbacks?

	if (RuntimeCachePolicy.bUseCurveTableCacheVersion)
	{
		UCurveTable::InvalidateAllCachedCurves();
	}
	else
	{
		Cache->CurrentCacheVersion++;
	}

	OnCacheVersionInvalidated().Broadcast(this);
}

FDataRegistryCacheVersionCallback& UDataRegistry::OnCacheVersionInvalidated()
{
	return OnCacheVersionInvalidatedCallback;
}

bool UDataRegistry::ResolveDataRegistryId(FDataRegistryLookup& OutLookup, const FDataRegistryId& ItemId, const uint8** PrecachedDataPtr) const
{
	OutLookup.Reset();

	for (int32 i = 0; i < RuntimeSources.Num(); i++)
	{
		const UDataRegistrySource* Source = RuntimeSources[i];
		if (!Source)
		{
			continue;
		}
		
		FName ResolvedName = MapIdToResolvedName(ItemId, Source);
		EDataRegistryAvailability Availability = ResolvedName == NAME_None ? EDataRegistryAvailability::DoesNotExist : Source->GetItemAvailability(ResolvedName, PrecachedDataPtr);

		// If we know it doesn't exist, don't add this to the lookup
		if (Availability != EDataRegistryAvailability::DoesNotExist)
		{
			OutLookup.AddEntry(i, ResolvedName);
		}
		
		if (Availability == EDataRegistryAvailability::PreCached)
		{
			// If the lookup has prior entries, not safe to use the in memory ptr as it may be out of date
			if (PrecachedDataPtr && OutLookup.GetNum() > 1)
			{
				*PrecachedDataPtr = nullptr;
			}

			// If we found it in memory, we're done
			return true;
		}

		// Otherwise keep going
	}
	return OutLookup.IsValid();
}

void UDataRegistry::GetPossibleRegistryIds(TArray<FDataRegistryId>& OutRegistryIds, bool bSortForDisplay) const
{
	TSet<FDataRegistryId> IdSet;
	TArray<FName> NameArray;

	FDataRegistryIdFormat TempIdFormat = GetIdFormat();
	if (TempIdFormat.BaseGameplayTag.IsValid())
	{
		// Every key inside the top level is a valid id
		UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();

		FGameplayTagContainer ChildTags = TagManager.RequestGameplayTagChildren(TempIdFormat.BaseGameplayTag);

		for (const FGameplayTag& ChildTag : ChildTags)
		{
			OutRegistryIds.Emplace(RegistryType, ChildTag.GetTagName());
		}
	}
	else
	{
		for (const UDataRegistrySource* Source : RuntimeSources)
		{
			if (Source)
			{
				NameArray.Reset();
				Source->GetResolvedNames(NameArray);

				for (const FName& ResolvedName : NameArray)
				{
					AddAllIdsForResolvedName(IdSet, ResolvedName, Source);
				}
			}
		}

		OutRegistryIds = IdSet.Array();

		if (bSortForDisplay)
		{
			// By default sort it alphabetically
			OutRegistryIds.Sort([&](const FDataRegistryId& A, const FDataRegistryId& B)
			{
				return A.ItemName.LexicalLess(B.ItemName);
			});
		}
	}
}

#if WITH_EDITOR

void UDataRegistry::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// TODO Only if major properties change?

	EditorRefreshRegistry();
}

void UDataRegistry::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// We possibly were moved to a new package, so refresh
	EditorRefreshRegistry();
}

void UDataRegistry::EditorRefreshRegistry()
{
	UDataRegistrySubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();

	if (RegistryType == FDataRegistryType::CustomContextType.GetName())
	{
		UE_LOGF(LogDataRegistry, Error, "Invalid registry type name %ls, subsystem not registered!", *RegistryType.ToString());
		return;
	}

	if (Subsystem && Subsystem->AreRegistriesInitialized())
	{
		// Deinitialize so we can readd
		Deinitialize();

		// First refresh our editor-defined sources
		for (int32 i = 0; i < DataSources.Num(); i++)
		{
			// Empty one are allowed to exist as they may still be being edited
			UDataRegistrySource* Source = DataSources[i];
			if (Source)
			{
				Source->EditorRefreshSource();
			}
		}

		// See if this is registered with subsystem, if not re-register
		UDataRegistry* FoundRegistry = Subsystem->GetRegistryForType(RegistryType);

		if (FoundRegistry != this)
		{
			Subsystem->RefreshRegistryMap();
			FoundRegistry = Subsystem->GetRegistryForType(RegistryType);
		}

		if (FoundRegistry == this)
		{
			// May need to reinitialize
			if (!IsInitialized())
			{
				Initialize();
			}
			else
			{
				RefreshRuntimeSources();
				ResetRuntimeState();
			}
		}
		else
		{
			Deinitialize();
		}
	}
	else
	{
		UE_LOGF(LogDataRegistry, Error, "Cannot validate DataRegistry %ls, subsystem not registered!", *GetPathName());
	}
}

void UDataRegistry::GetAllSourceItems(TArray<FDataRegistrySourceItemId>& OutSourceItems) const
{
	TSet<FDataRegistryId> IdSet;
	TArray<FName> NameArray;
	for (int32 i = 0; i < RuntimeSources.Num(); i++)
	{
		const UDataRegistrySource* Source = RuntimeSources[i];
		if (Source)
		{
			NameArray.Reset();
			Source->GetResolvedNames(NameArray);

			for (const FName& ResolvedName : NameArray)
			{
				IdSet.Reset();
				AddAllIdsForResolvedName(IdSet, ResolvedName, Source);

				for (FDataRegistryId& ItemId : IdSet)
				{
					FDataRegistrySourceItemId SourceId;
					SourceId.ItemId = ItemId;
					SourceId.SourceResolvedName = ResolvedName;
					
					// Generate a direct lookup
					SourceId.CacheLookup.AddEntry(i, ResolvedName);
					SourceId.CachedSource = Source;

					OutSourceItems.Add(SourceId);
				}
				
			}
		}
	}
}

bool UDataRegistry::BatchAcquireSourceItems(TArray<FDataRegistrySourceItemId>& SourceItems, FDataRegistryBatchAcquireCallback DelegateToCall)
{
	bool bAnySuccess = false;

	FDataRegistryBatchRequest& NewRequest = Cache->CreateNewBatchRequest();
	NewRequest.Callback = DelegateToCall;

	for (const FDataRegistrySourceItemId& SourceItem : SourceItems)
	{
		bool bSuccess = AcquireItemInternal(SourceItem.ItemId, SourceItem.CacheLookup, FDataRegistryItemAcquiredCallback(), NewRequest.RequestId);

		if (bSuccess)
		{
			bAnySuccess = true;
			NewRequest.RequestedItems.Emplace(SourceItem.ItemId, SourceItem.CacheLookup);
			NewRequest.RemainingAcquires.Add(SourceItem.CacheLookup);
		}
	}

	if (!bAnySuccess)
	{
		// Nothing started so just kill request now
		Cache->RemoveBatchRequest(NewRequest.RequestId);
	}

	return bAnySuccess;
}

#endif

bool UDataRegistry::AcquireItem(const FDataRegistryId& ItemId, FDataRegistryItemAcquiredCallback DelegateToCall)
{
	const uint8* InMemoryData = nullptr;
	FDataRegistryLookup TempLookup;
	if (ResolveDataRegistryId(TempLookup, ItemId, &InMemoryData) && TempLookup.IsValid())
	{
		return AcquireItemInternal(ItemId, TempLookup, DelegateToCall, FDataRegistryRequestId());
	}

	return false;
}

bool UDataRegistry::BatchAcquireItems(const TArray<FDataRegistryId>& ItemIds, FDataRegistryBatchAcquireCallback DelegateToCall)
{
	bool bAnySuccess = false;

	const uint8* InMemoryData = nullptr;
	FDataRegistryLookup TempLookup;

	FDataRegistryBatchRequest& NewRequest = Cache->CreateNewBatchRequest();
	NewRequest.Callback = DelegateToCall;

	for (const FDataRegistryId& ItemId : ItemIds)
	{
		if (ResolveDataRegistryId(TempLookup, ItemId, &InMemoryData) && TempLookup.IsValid())
		{
			bool bSuccess = AcquireItemInternal(ItemId, TempLookup, FDataRegistryItemAcquiredCallback(), NewRequest.RequestId);

			if (bSuccess)
			{
				bAnySuccess = true;
				NewRequest.RequestedItems.Emplace(ItemId, TempLookup);
				NewRequest.RemainingAcquires.Add(TempLookup);
			}
		}
	}

	if (!bAnySuccess)
	{
		// Nothing started so just kill request now
		Cache->RemoveBatchRequest(NewRequest.RequestId);
	}

	return bAnySuccess;
}

bool UDataRegistry::AcquireItemInternal(const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup, FDataRegistryItemAcquiredCallback DelegateToCall, const FDataRegistryRequestId& BatchRequestId)
{
	// TODO add handling for no resource already-loaded case, just need to fake callbacks
	FCachedDataRegistryItem& CacheEntry = Cache->GetOrCreateCacheEntry(Lookup);

	CacheEntry.LastAccessTime = GetCurrentTime();

	FCachedDataRegistryRequest& NewRequest = CacheEntry.ActiveRequests.Emplace_GetRef();
	NewRequest.RequestedId = ItemId;
	NewRequest.Callback = DelegateToCall;
	NewRequest.BatchId = BatchRequestId;
		
	if (CacheEntry.AcquireStatus == EDataRegistryAcquireStatus::NotStarted)
	{
		HandleCacheEntryState(Lookup, CacheEntry);
	}
	else if (CacheEntry.AcquireStatus == EDataRegistryAcquireStatus::AcquireFinished)
	{
		// Schedule success for next frame
		FStreamableHandle::ExecuteDelegate(FStreamableDelegate::CreateUObject(this, &UDataRegistry::DelayedHandleSuccessCallbacks, Lookup));
	}
	else if (CacheEntry.AcquireStatus == EDataRegistryAcquireStatus::AcquireError)
	{
		// Or should this call error callback next frame?
		return false;
	}

	return true;
}

FDataRegistryCacheGetResult UDataRegistry::GetCachedItemRawFromLookup(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup) const
{
	OutItemMemory = nullptr;
	OutItemStruct = GetItemStruct();

	const FCachedDataRegistryItem* FoundCache = Cache->GetCacheEntry(Lookup);

	if (FoundCache && FoundCache->ItemMemory)
	{
		OutItemMemory = FoundCache->ItemMemory;
		return GetCacheResultVersion();
	}

	// If not in cache, try to do an immediate resolve
	// Only the first resolve could possibly be valid now, if it's a slow or complicated resolve it will be in the cache
	FName ResolvedName;
	UDataRegistrySource* SourceToCheck = LookupSource(ResolvedName, Lookup, 0);

	if (SourceToCheck)
	{
		EDataRegistryAvailability Availability = SourceToCheck->GetItemAvailability(ResolvedName, &OutItemMemory);
		if (Availability == EDataRegistryAvailability::PreCached)
		{
			return GetCacheResultVersion();
		}
	}

	OutItemMemory = nullptr;
	return FDataRegistryCacheGetResult();
}

FDataRegistryCacheGetResult UDataRegistry::GetCachedItemRaw(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId) const
{
	OutItemMemory = nullptr;
	OutItemStruct = GetItemStruct();

	// FoundCache may return null, but fill in OutItemMemory, if it is immediately available
	const FCachedDataRegistryItem* FoundCache = FindCachedData(ItemId, &OutItemMemory);
	if (FoundCache && !OutItemMemory)
	{
		OutItemMemory = FoundCache->ItemMemory;
	}

	if (OutItemMemory != nullptr)
	{
		return GetCacheResultVersion();
	}

	return FDataRegistryCacheGetResult();
}

FDataRegistryCacheGetResult UDataRegistry::GetCachedCurveRaw(const FRealCurve*& OutCurve, const FDataRegistryId& ItemId) const
{
	static UScriptStruct* CurveScriptStruct = FRealCurve::StaticStruct();

	const uint8* TempItemMemory = nullptr;
	const UScriptStruct* TempItemStuct = nullptr;
	FDataRegistryCacheGetResult CacheResult = GetCachedItemRaw(TempItemMemory, TempItemStuct, ItemId);
	
	if (CacheResult && TempItemMemory && TempItemStuct->IsChildOf(CurveScriptStruct))
	{
		OutCurve = (const FRealCurve*)TempItemMemory;
		return CacheResult;
	}

	OutCurve = nullptr;
	return FDataRegistryCacheGetResult();
}

UDataRegistrySource* UDataRegistry::LookupSource(FName& OutResolvedName, const FDataRegistryLookup& Lookup, int32 LookupIndex) const
{
	uint8 SourceIndex = 255;
	if (Lookup.GetEntry(SourceIndex, OutResolvedName, LookupIndex))
	{
		if (RuntimeSources.IsValidIndex(SourceIndex))
		{
			return RuntimeSources[SourceIndex];
		}
	}
	return nullptr;
}

void UDataRegistry::GetItemNames(TArray<FName>& ItemNames) const
{
	TMap<FDataRegistryId, const uint8*> CachedItemMap;
	const UScriptStruct* ItemTypeStruct = nullptr;

	const FDataRegistryCacheGetResult Result = GetAllCachedItems(CachedItemMap, ItemTypeStruct);
	if (!Result.WasFound())
	{
		UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry Data found  Registry:%ls", __FUNCTION__, *GetPathName());
		return;
	}

	for (TPair<FDataRegistryId, const uint8*>& CachedItem : CachedItemMap)
	{
		const FDataRegistryId& RegistryId = CachedItem.Key;
		ItemNames.Add(RegistryId.ItemName);
	}
}

FDataRegistryCacheGetResult UDataRegistry::GetAllCachedItems(TMap<FDataRegistryId, const uint8*>& OutItemMap, const UScriptStruct*& OutItemStruct) const
{
	TSet<FDataRegistryId> IdSet;
	TArray<FName> NameArray;

	OutItemStruct = GetItemStruct();

	// First add all the locally cached items
	bool bFoundAny = false;
	for (const TPair<FDataRegistryLookup, TUniquePtr<FCachedDataRegistryItem>>& Pair : Cache->LookupCache)
	{
		FCachedDataRegistryItem* Item = Pair.Value.Get();

		if (Item && Item->ItemMemory && Item->AcquireStatus == EDataRegistryAcquireStatus::AcquireFinished)
		{
			bFoundAny = true;
			FName ResolvedName;
			UDataRegistrySource* Source = LookupSource(ResolvedName, Pair.Key, Item->AcquireLookupIndex);

			if (Source)
			{
				// If we found the source, then convert the resolved name back into registry ids
				IdSet.Reset();
				AddAllIdsForResolvedName(IdSet, ResolvedName, Source);

				for (const FDataRegistryId& CachedId : IdSet)
				{
					// If this isn't already in the map add it, same pointer can be mapped to multiple ids
					const uint8*& FoundPtr = OutItemMap.FindOrAdd(CachedId);

					if (!FoundPtr)
					{
						bFoundAny = true;
						FoundPtr = Item->ItemMemory;
					}
				}
			}
		}
	}

	// Now iterate looking for precached sources, starting from highest priority source
	// TODO if this is a performance issue could add a new function to source
	for (const UDataRegistrySource* Source : RuntimeSources)
	{
		if (Source)
		{
			NameArray.Reset();
			Source->GetResolvedNames(NameArray);

			for (const FName& ResolvedName : NameArray)
			{
				const uint8* OutItemMemory = nullptr;
				EDataRegistryAvailability Availability = Source->GetItemAvailability(ResolvedName, &OutItemMemory);
				if (Availability == EDataRegistryAvailability::PreCached)
				{
					IdSet.Reset();
					AddAllIdsForResolvedName(IdSet, ResolvedName, Source);

					for (const FDataRegistryId& CachedId : IdSet)
					{
						// If this isn't already in the map add it, same pointer can be mapped to multiple ids
						const uint8*& FoundPtr = OutItemMap.FindOrAdd(CachedId);

						if (!FoundPtr)
						{
							bFoundAny = true;
							FoundPtr = OutItemMemory;
						}
					}
				}
			}
		}
	}

	if (bFoundAny)
	{
		return GetCacheResultVersion();
	}
	else
	{
		return FDataRegistryCacheGetResult();
	}
}

void UDataRegistry::GetChildRuntimeSources(UDataRegistrySource* ParentSource, TArray<UDataRegistrySource*>& ChildSources) const
{
	ChildSources.Reset();
	for (int32 i = 0; i < RuntimeSources.Num(); i++)
	{
		UDataRegistrySource* Source = RuntimeSources[i];
		if (Source && Source->GetOriginalSource() == ParentSource)
		{
			ChildSources.Add(Source);
		}
	}
}

int32 UDataRegistry::GetSourceIndex(const UDataRegistrySource* Source, bool bUseRuntimeSources /*= true*/) const
{
	int32 FoundIndex = INDEX_NONE;
	if (bUseRuntimeSources)
	{
		RuntimeSources.Find(const_cast<UDataRegistrySource *>(Source), FoundIndex);
	}
	else
	{
		DataSources.Find(const_cast<UDataRegistrySource*>(Source), FoundIndex);
	}
	return FoundIndex;
}

void UDataRegistry::RefreshRuntimeSources()
{
	TArray<UDataRegistrySource*> OldRuntimeSources = RuntimeSources;
	RuntimeSources.Reset();

	// Get new sources in order
	for (int32 i = 0; i < DataSources.Num(); i++)
	{
		UDataRegistrySource* Source = DataSources[i];
		if (Source)
		{
			if (IsInitialized() && !Source->IsInitialized())
			{
				Source->Initialize();
			}

			Source->RefreshRuntimeSources();
			Source->AddRuntimeSources(MutableView(RuntimeSources));
		}
	}

	// Deinitialize any orphaned sources
	for (int32 i = 0; i < OldRuntimeSources.Num(); i++)
	{
		UDataRegistrySource* OldSource = OldRuntimeSources[i];
		if (IsValid(OldSource) && OldSource->GetOuter() == this && OldSource->IsInitialized() && !RuntimeSources.Contains(OldSource))
		{
			OldSource->Deinitialize();

			if (OldSource->IsTransientSource())
			{
				// Transient sources should have already been removed during RefreshRuntimeSources
				UE_LOGF(LogDataRegistry, Warning, "RefreshRuntimeSources found orphaned transient source %ls for registry %ls!", *OldSource->GetPathName(), *RegistryType.ToString());
			}
		}
	}

	// Need to clear the cache as lookups are now invalid
	// TODO: Also refresh or cancel in-progress async requests. Possibly only conditionally clear cache
	Cache->ClearCache(false);

	InvalidateCacheVersion();
	bNeedsRuntimeRefresh = false;
}

float UDataRegistry::GetCurrentTime()
{
	// Use application time so it works in editor and game
	return (float)(FApp::GetCurrentTime() - GStartTime);
}

UE_NO_PROFILE_ATTRIBUTE	// this is a workaround for a bug in clang that applies musttail attribute to Source->TimerUpdate call in some cases
void UDataRegistry::TimerUpdate()
{
	if (!bIsInitialized)
	{
		return;
	}

	ApplyCachePolicy();
	float CurrentTime = GetCurrentTime();

	for (int32 i = 0; i < RuntimeSources.Num(); i++)
	{
		UDataRegistrySource* Source = RuntimeSources[i];
		if (Source)
		{
			Source->TimerUpdate(CurrentTime, TimerUpdateFrequency);
		}
	}
}

FName UDataRegistry::MapIdToResolvedName(const FDataRegistryId& ItemId, const UDataRegistrySource* RegistrySource) const
{
	// Try resolver stack before falling back to raw name
	FName ResolvedName;
	FDataRegistryResolver* FoundResolver = FDataRegistryResolverScope::ResolveNameFromId(ResolvedName, ItemId, this, RegistrySource);
	if (FoundResolver)
	{
		return ResolvedName;
	}

	return ItemId.ItemName;
}

void UDataRegistry::AddAllIdsForResolvedName(TSet<FDataRegistryId>& PossibleIds, const FName& ResolvedName, const UDataRegistrySource* RegistrySource) const
{
	PossibleIds.Add(FDataRegistryId(RegistryType, ResolvedName));
}

const FCachedDataRegistryItem* UDataRegistry::FindCachedData(const FDataRegistryId& ItemId, const uint8** PrecachedDataPtr) const
{
	FDataRegistryLookup TempLookup;
	if (ResolveDataRegistryId(TempLookup, ItemId, PrecachedDataPtr))
	{
		return Cache->GetCacheEntry(TempLookup);
	}
	return nullptr;
}

void UDataRegistry::HandleCacheEntryState(const FDataRegistryLookup& Lookup, FCachedDataRegistryItem& CachedEntry)
{
	if (CachedEntry.AcquireStatus == EDataRegistryAcquireStatus::NotStarted)
	{
		CachedEntry.AcquireLookupIndex = 0;
		CachedEntry.AcquireStatus = EDataRegistryAcquireStatus::WaitingForInitialAcquire;
	}

	if (CachedEntry.AcquireStatus == EDataRegistryAcquireStatus::WaitingForInitialAcquire)
	{
		FName ResolvedName;
		UDataRegistrySource* SourceToCheck = LookupSource(ResolvedName, Lookup, CachedEntry.AcquireLookupIndex);

		if (SourceToCheck)
		{
			FDataRegistrySourceAcquireRequest AcquireRequest;
			AcquireRequest.Lookup = Lookup;
			AcquireRequest.LookupIndex = CachedEntry.AcquireLookupIndex;

			SourceToCheck->AcquireItem(MoveTemp(AcquireRequest));
		}
		else
		{
			// Done checking sources, move on
			CachedEntry.AcquireStatus = EDataRegistryAcquireStatus::AcquireError;
		}
	}

	if (CachedEntry.AcquireStatus == EDataRegistryAcquireStatus::InitialAcquireFinished)
	{
		// TODO add resource loading
		CachedEntry.AcquireStatus = EDataRegistryAcquireStatus::AcquireFinished;
	}

	if (CachedEntry.AcquireStatus == EDataRegistryAcquireStatus::AcquireFinished)
	{
		// Call callbacks
		HandleAcquireCallbacks(Lookup, CachedEntry);
	}

	if (CachedEntry.AcquireStatus == EDataRegistryAcquireStatus::AcquireError)
	{
		// Call callbacks
		HandleAcquireCallbacks(Lookup, CachedEntry);
	}
}


void UDataRegistry::HandleAcquireCallbacks(const FDataRegistryLookup& Lookup, FCachedDataRegistryItem& CachedEntry)
{
	// Create with default id, then fill in
	FDataRegistryAcquireResult ResultStruct(FDataRegistryId(), Lookup, CachedEntry.AcquireStatus, GetItemStruct(), CachedEntry.ItemMemory);
	
	int32 NumCallbacks = CachedEntry.ActiveRequests.Num();
	for (int32 i = 0; i < NumCallbacks; i++)
	{
		FCachedDataRegistryRequest& Request = CachedEntry.ActiveRequests[i];
		ResultStruct.ItemId = Request.RequestedId;

		if (Request.BatchId.IsValid())
		{
			UpdateBatchRequest(Request.BatchId, ResultStruct);
		}

		Request.Callback.ExecuteIfBound(ResultStruct);

		// Callback could modify array
	}


	CachedEntry.ActiveRequests.RemoveAt(0, NumCallbacks);
}

void UDataRegistry::UpdateBatchRequest(const FDataRegistryRequestId& RequestId, const FDataRegistryAcquireResult& Result)
{
	FDataRegistryBatchRequest* FoundRequest = Cache->GetBatchRequest(RequestId);

	if (FoundRequest)
	{
		FDataRegistryAcquireResult::UpdateAcquireStatus(FoundRequest->BatchStatus, Result.Status);

		FoundRequest->RemainingAcquires.Remove(Result.ResolvedLookup);

		if (FoundRequest->RemainingAcquires.Num() == 0)
		{
			FoundRequest->Callback.ExecuteIfBound(FoundRequest->BatchStatus);

			Cache->RemoveBatchRequest(RequestId);
			// This invalidates FoundRequest
		}
	}
}

void UDataRegistry::DelayedHandleSuccessCallbacks(FDataRegistryLookup Lookup)
{
	FCachedDataRegistryItem* FoundCache = Cache->GetCacheEntry(Lookup);

	if (FoundCache && FoundCache->AcquireStatus == EDataRegistryAcquireStatus::AcquireFinished)
	{
		HandleAcquireCallbacks(Lookup, *FoundCache);
	}
}

void UDataRegistry::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataRegistry* This = CastChecked<UDataRegistry>(InThis);
	const UScriptStruct* ItemStruct = This->GetItemStruct();

	// Need to emit references for referenced items (unless there's no properties that reference UObjects)
	if (ItemStruct != nullptr && ItemStruct->RefLink != nullptr && This->Cache->LookupCache.Num() > 0)
	{
		for (TPair<FDataRegistryLookup, TUniquePtr<FCachedDataRegistryItem>>& Pair : This->Cache->LookupCache)
		{
			FCachedDataRegistryItem* Item = Pair.Value.Get();

			if (Item && Item->ItemMemory)
			{
				Collector.AddPropertyReferencesWithStructARO(ItemStruct, Item->ItemMemory, This);
			}
		}
	}

	Super::AddReferencedObjects(This, Collector);
}

void UDataRegistry::HandleAcquireResult(const FDataRegistrySourceAcquireRequest& Request, EDataRegistryAcquireStatus Status, uint8* ItemMemory, UDataRegistrySource* Source)
{
	// Make sure this is a valid state
	FCachedDataRegistryItem* FoundCache = Cache->GetCacheEntry(Request.Lookup);

	if (FoundCache)
	{
		if (FoundCache->AcquireStatus == EDataRegistryAcquireStatus::WaitingForInitialAcquire)
		{
			FName ResolvedName;
			UDataRegistrySource* CurrentSource = LookupSource(ResolvedName, Request.Lookup, FoundCache->AcquireLookupIndex);
			if (CurrentSource == Source)
			{
				if (Status == EDataRegistryAcquireStatus::InitialAcquireFinished)
				{
					// Add to cache
					FoundCache->ItemMemory = ItemMemory;
					FoundCache->ItemSource = Source;
					FoundCache->AcquireStatus = EDataRegistryAcquireStatus::InitialAcquireFinished;

					// TODO Do we need to defer this for callback safety?
					InvalidateCacheVersion();
				}
				else
				{
					// TODO better error handling
					FoundCache->AcquireLookupIndex++;
					// Try next source
				}

				HandleCacheEntryState(Request.Lookup, *FoundCache);
			}
			else
			{
				UE_LOGF(LogDataRegistry, Error, "HandleAcquireResult called for %ls for wrong source %ls!", *Request.Lookup.ToString(), *GetNameSafe(Source));
			}
		}
		else
		{
			UE_LOGF(LogDataRegistry, Error, "HandleAcquireResult called for %ls in invalid state %d!", *Request.Lookup.ToString(), int(FoundCache->AcquireStatus));
		}
	}
	else 
	{
		UE_LOGF(LogDataRegistry, Error, "HandleAcquireResult called for %ls with invalid cache!", *Request.Lookup.ToString());
	}
	
}

class FTimerManager* UDataRegistry::GetTimerManager()
{
	check(UAssetManager::IsInitialized());
	return UAssetManager::Get().GetTimerManager();
}

#undef LOCTEXT_NAMESPACE
