// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryId.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistryId)

#define LOCTEXT_NAMESPACE "DataRegistry"

const FName FDataRegistryType::ItemStructMetaData = FName(TEXT("ItemStruct"));
const FDataRegistryType FDataRegistryType::CustomContextType = FDataRegistryType(TEXT("CustomContext"));

bool FDataRegistryType::ExportTextItem(FString& ValueStr, FDataRegistryType const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!(PortFlags & PPF_Delimited))
	{
		ValueStr += ToString();
	}
	else
	{
		ValueStr += FString::Printf(TEXT("\"%s\""), *ToString().ReplaceCharWithEscapedChar());
	}

	return true;
}

bool FDataRegistryType::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// This handles both quoted and unquoted
	FString ImportedString = TEXT("");
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, ImportedString, 1);

	if (!NewBuffer)
	{
		return false;
	}

	*this = FDataRegistryType(*ImportedString);
	Buffer = NewBuffer;

	return true;
}

bool FDataRegistryType::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		FName InName;
		Slot << InName;
		*this = FDataRegistryType(InName);
		return true;
	}
	else if (Tag.Type == NAME_StrProperty)
	{
		FString InString;
		Slot << InString;
		*this = FDataRegistryType(*InString);
		return true;
	}

	return false;
}

bool FDataRegistryId::ExportTextItem(FString& ValueStr, FDataRegistryId const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!(PortFlags & PPF_Delimited))
	{
		ValueStr += ToString();
	}
	else
	{
		ValueStr += FString::Printf(TEXT("\"%s\""), *ToString().ReplaceCharWithEscapedChar());
	}

	return true;
}

bool FDataRegistryId::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// This handles both quoted and unquoted
	FString ImportedString = TEXT("");
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, ImportedString, 1);

	if (!NewBuffer)
	{
		return false;
	}

	*this = FDataRegistryId(ImportedString);
	Buffer = NewBuffer;

	return true;
}

bool FDataRegistryId::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		FName InName;
		Slot << InName;
		*this = FDataRegistryId(InName.ToString());
		return true;
	}
	else if (Tag.Type == NAME_StrProperty)
	{
		FString InString;
		Slot << InString;
		*this = FDataRegistryId(InString);
		return true;
	}

	return false;
}

void FDataRegistryId::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving() && IsValid())
	{
		Ar.MarkSearchableName(FDataRegistryId::StaticStruct(), FName(*ToString()));
	}
}

FString FDataRegistryId::ToString() const
{
	if (IsValid())
	{
		return FString::Printf(TEXT("%s:%s"), *RegistryType.ToString(), *ItemName.ToString());
	}
	else
	{
		return FString();
	}
}

FText FDataRegistryId::ToText() const
{
	if (IsValid())
	{
		return FText::AsCultureInvariant(ToString());
	}

	return LOCTEXT("NoneValue", "None");
}

#undef LOCTEXT_NAMESPACE


