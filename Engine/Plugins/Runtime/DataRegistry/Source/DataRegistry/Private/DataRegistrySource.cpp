// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistrySource.h"
#include "AssetRegistry/AssetData.h"
#include "DataRegistrySubsystem.h"
#include "Engine/AssetManager.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistrySource)

const UDataRegistry* UDataRegistrySource::GetRegistry() const
{
	return CastChecked<UDataRegistry>(GetOuter());
}

UDataRegistry* UDataRegistrySource::GetRegistry()
{
	return CastChecked<UDataRegistry>(GetOuter());
}

const UScriptStruct* UDataRegistrySource::GetItemStruct() const
{
	return GetRegistry()->GetItemStruct();
}

EDataRegistryAvailability UDataRegistrySource::GetSourceAvailability() const
{
	return EDataRegistryAvailability::DoesNotExist;
}

EDataRegistryAvailability UDataRegistrySource::GetItemAvailability(const FName& ResolvedName, const uint8** PrecachedDataPtr) const
{
	return EDataRegistryAvailability::DoesNotExist;
}

void UDataRegistrySource::GetResolvedNames(TArray<FName>& Names) const
{

}

bool UDataRegistrySource::IsInitialized() const
{
	return bIsInitialized;
}

bool UDataRegistrySource::Initialize()
{
	if (IsInitialized())
	{
		return false;
	}

	bIsInitialized = true;
	return true;
}

void UDataRegistrySource::Deinitialize()
{
	if (IsInitialized() && GetRegistry())
	{
		bIsInitialized = false;
	}
}

void UDataRegistrySource::RefreshRuntimeSources()
{

}

void UDataRegistrySource::ResetRuntimeState()
{

}

void UDataRegistrySource::TimerUpdate(float CurrentTime, float TimerUpdateFrequency)
{

}

void UDataRegistrySource::HandleAcquireResult(const FDataRegistrySourceAcquireRequest& Request, EDataRegistryAcquireStatus Status, uint8* ItemMemory)
{
	GetRegistry()->HandleAcquireResult(Request, Status, ItemMemory, this);
}

bool UDataRegistrySource::AcquireItem(FDataRegistrySourceAcquireRequest&& Request)
{
	return false;
}


FString UDataRegistrySource::GetDebugString() const
{
	return TEXT("InvalidSource");
}

FSoftObjectPath UDataRegistrySource::GetSourceAssetPath() const
{
	return FSoftObjectPath();
}

bool UDataRegistrySource::IsTransientSource() const
{
	return HasAnyFlags(RF_Transient);
}

bool UDataRegistrySource::ResetTransientSource()
{
	if (ensure(IsTransientSource() && !IsInitialized()))
	{
		// Mark that this is no longer part of a parent source, the calling function must remove it from any parent-child arrays
		ParentSource = nullptr;
		RemoveFromRoot();
		return true;
	}

	return false;
}

UDataRegistrySource* UDataRegistrySource::GetOriginalSource()
{
	return ParentSource ? ToRawPtr(ParentSource) : this;
}

bool UDataRegistrySource::IsSpecificAssetRegistered(const FSoftObjectPath& AssetPath) const
{
	return false;
}

EDataRegistryRegisterAssetResult UDataRegistrySource::RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority)
{
	return EDataRegistryRegisterAssetResult::NotRegistered;
}

bool UDataRegistrySource::UnregisterSpecificAsset(const FSoftObjectPath& AssetPath)
{
	return false;
}

int32 UDataRegistrySource::UnregisterAssetsWithPriority(int32 AssetPriority)
{
	return 0;
}

void UDataRegistrySource::AddRuntimeSources(TArray<UDataRegistrySource*>& OutRuntimeSources)
{
	if (IsInitialized())
	{
		OutRuntimeSources.Add(this);
	}
}

UDataRegistrySource* UDataRegistrySource::CreateTransientSource(TSubclassOf<UDataRegistrySource> SourceClass)
{
	if (!SourceClass || SourceClass->HasAnyClassFlags(CLASS_Abstract))
	{
		ensureMsgf(0, TEXT("GetChildSourceClass must return non-abstract class!"));
		return nullptr;
	}

	UDataRegistrySource* ChildSource = NewObject<UDataRegistrySource>(GetRegistry(), *SourceClass, NAME_None, RF_Transient);
	ChildSource->ParentSource = this;
	
	if (GUObjectArray.IsDisregardForGC(this))
	{
		ChildSource->AddToRoot();
	}

	return ChildSource;
}


#if WITH_EDITOR

void UDataRegistrySource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	EditorRefreshSource();
}

void UDataRegistrySource::EditorRefreshSource()
{

}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSubclassOf<UDataRegistrySource> UMetaDataRegistrySource::GetChildSourceClass() const
{
	return UDataRegistrySource::StaticClass();
}

bool UMetaDataRegistrySource::SetDataForChild(FName SourceName, UDataRegistrySource* ChildSource)
{
	ensureMsgf(0, TEXT("Must Be Implemented in Subclass"));
	return false;
}

void UMetaDataRegistrySource::DetermineRuntimeNames(TArray<FName>& OutRuntimeNames)
{
	// Try default scan behavior
	OutRuntimeNames.Reset();

	UDataRegistrySubsystem* Subsystem = UDataRegistrySubsystem::Get();
	TArray<FAssetData> AssetDataList;
	const UScriptStruct* ItemStruct = GetItemStruct();

	if (!ItemStruct || !Subsystem)
	{
		return;
	}

	if (EnumHasAnyFlags(AssetUsage, EMetaDataRegistrySourceAssetUsage::SearchAssets))
	{
		UAssetManager& AssetManager = UAssetManager::Get();
		FAssetManagerSearchRules TempRules = SearchRules;
		bool bExpandedVirtual = false;
		if (!TempRules.bSkipVirtualPathExpansion)
		{
			bExpandedVirtual = AssetManager.ExpandVirtualPaths(TempRules.AssetScanPaths);
			TempRules.bSkipVirtualPathExpansion = true;
		}

		AssetManager.SearchAssetRegistryPaths(AssetDataList, TempRules);

		if (bExpandedVirtual)
		{
			// Need to register for changes to virtual paths
			if (!NewAssetSearchRootHandle.IsValid())
			{
				NewAssetSearchRootHandle = AssetManager.Register_OnAddedAssetSearchRoot(FOnAddedAssetSearchRoot::FDelegate::CreateUObject(this, &UMetaDataRegistrySource::OnNewAssetSearchRoot));
			}
		}
	}

	if (EnumHasAnyFlags(AssetUsage, EMetaDataRegistrySourceAssetUsage::RegisterAssets))
	{
		for (const FRegisteredAsset& RegisteredPair : SpecificRegisteredAssets)
		{
			AssetDataList.Add(RegisteredPair.Key);
		}
	}

	for (const FAssetData& AssetData : AssetDataList)
	{
		if (DoesAssetPassFilter(AssetData, false))
		{
			OutRuntimeNames.Add(FName(*AssetData.GetObjectPathString()));
		}
	}
}

bool UMetaDataRegistrySource::DoesAssetPassFilter(const FAssetData& AssetData, bool bNewRegisteredAsset)
{
	return false;
}

void UMetaDataRegistrySource::RefreshRuntimeSources()
{
	if (!IsInitialized())
	{
		// This will clear out all active runtime children
		RuntimeNames.Reset();
	}
	else
	{
		DetermineRuntimeNames(RuntimeNames);

		for (FName SourceName : RuntimeNames)
		{
			TObjectPtr<UDataRegistrySource>* FoundSource = RuntimeChildren.Find(SourceName);
			if (FoundSource && *FoundSource)
			{
				SetDataForChild(SourceName, *FoundSource);
				ensureMsgf((*FoundSource)->IsInitialized(), TEXT("Runtime source %s was not initialized after refresh!"), *FoundSource->GetPathName());
			}
			else
			{
				UDataRegistrySource* NewSource = CreateTransientSource(GetChildSourceClass());
				SetDataForChild(SourceName, NewSource);
				RuntimeChildren.Add(SourceName, NewSource);
				NewSource->Initialize();
			}
		}
	}

	// Clear out old undesired children and deinitialize them
	for (auto MapIt = RuntimeChildren.CreateIterator(); MapIt; ++MapIt)
	{
		if (!RuntimeNames.Contains(MapIt.Key()))
		{
			if (MapIt.Value())
			{
				MapIt.Value()->Deinitialize();
				MapIt.Value()->ResetTransientSource();
			}
			MapIt.RemoveCurrent();
		}
	}
}

void UMetaDataRegistrySource::AddRuntimeSources(TArray<UDataRegistrySource*>& OutRuntimeSources)
{
	if (IsInitialized())
	{
		for (FName SourceName : RuntimeNames)
		{
			TObjectPtr<UDataRegistrySource>* FoundSource = RuntimeChildren.Find(SourceName);
			if (FoundSource && *FoundSource)
			{
				OutRuntimeSources.Add(*FoundSource);
			}
		}
	}
}

bool UMetaDataRegistrySource::IsSpecificAssetRegistered(const FSoftObjectPath& AssetPath) const
{
	// Look for existing asset
	for (const FRegisteredAsset& Existing : SpecificRegisteredAssets)
	{
		if (AssetPath == Existing.Key.ToSoftObjectPath())
		{
			return true;
		}
	}

	return false;
}

EDataRegistryRegisterAssetResult UMetaDataRegistrySource::RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority)
{
	bool bMadeChange = false;
	
	// Look for existing asset
	for (FRegisteredAsset& Existing : SpecificRegisteredAssets)
	{
		if (AssetData == Existing.Key)
		{
			if (Existing.Value == AssetPriority)
			{
				// Nothing to do
				return EDataRegistryRegisterAssetResult::AssetAlreadyRegistered;
			}
			bMadeChange = true;
			Existing.Value = AssetPriority;
			break;
		}
	}

	if (!bMadeChange && EnumHasAnyFlags(AssetUsage, EMetaDataRegistrySourceAssetUsage::RegisterAssets))
	{
		if (DoesAssetPassFilter(AssetData, true))
		{
			bMadeChange = true;
			SpecificRegisteredAssets.Emplace(AssetData, AssetPriority);
		}
	}

	if (bMadeChange)
	{
		SortRegisteredAssets();
	}

	return bMadeChange ? EDataRegistryRegisterAssetResult::RegisteredSuccesfully : EDataRegistryRegisterAssetResult::NotRegistered;
}

bool UMetaDataRegistrySource::UnregisterSpecificAsset(const FSoftObjectPath& AssetPath)
{
	for (int32 i = 0; i < SpecificRegisteredAssets.Num(); i++)
	{
		if (AssetPath == SpecificRegisteredAssets[i].Key.ToSoftObjectPath())
		{
			// No need to re-sort for a simple removal
			SpecificRegisteredAssets.RemoveAt(i);
			return true;
		}
	}

	return false;
}

int32 UMetaDataRegistrySource::UnregisterAssetsWithPriority(int32 AssetPriority)
{
	int32 NumberUnregistered = 0;
	for (int32 i = 0; i < SpecificRegisteredAssets.Num(); i++)
	{
		if (AssetPriority == SpecificRegisteredAssets[i].Value)
		{
			// No need to re-sort for a simple removal
			SpecificRegisteredAssets.RemoveAt(i);
			i--;
			NumberUnregistered++;
		}
	}

	return NumberUnregistered;
}

void UMetaDataRegistrySource::OnNewAssetSearchRoot(const FString& SearchRoot)
{
	// By default tell our registry to refresh
	if (IsInitialized())
	{
		GetRegistry()->MarkRuntimeDirty();
		GetRegistry()->RuntimeRefreshIfNeeded();
	}
}

void UMetaDataRegistrySource::SortRegisteredAssets()
{
	// Sort by priority, game subclasses can do something special if they want to
	SpecificRegisteredAssets.StableSort([](const FRegisteredAsset& A, const FRegisteredAsset& B)
	{
		return A.Value > B.Value;
	});
}

