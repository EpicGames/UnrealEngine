// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistrySource_CurveTable.h"

#include "DataRegistryTypesPrivate.h"
#include "DataRegistrySettings.h"
#include "Engine/AssetManager.h"
#include "Engine/CurveTable.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectThreadContext.h"
#include "Engine/StreamableManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistrySource_CurveTable)

void UDataRegistrySource_CurveTable::SetSourceTable(const TSoftObjectPtr<UCurveTable>& InSourceTable, const FDataRegistrySource_DataTableRules& InTableRules)
{
	if (ensure(IsTransientSource() || GIsEditor))
	{
		SourceTable = InSourceTable;
		TableRules = InTableRules;
		SetCachedTable(false);
	}
}

void UDataRegistrySource_CurveTable::SetCachedTable(bool bForceLoad /*= false*/)
{
	if (CachedTable)
	{
		CachedTable->OnCurveTableChanged().RemoveAll(this);
	}

	CachedTable = nullptr;
	UCurveTable* FoundTable = SourceTable.Get();

	if (!FoundTable && (bForceLoad || TableRules.bPrecacheTable))
	{
		if (FUObjectThreadContext::Get().IsRoutingPostLoad)
		{
			// It is not safe to try and preload during a postload, this should only happen if PreloadTable is null
			UE_LOGF(LogDataRegistry, Log, "Failed to preload table %ls for source %ls, PreloadTable is %ls", *SourceTable.ToString(), *GetPathName(), *GetNameSafe(PreloadTable));
		}
		else 
		{
			if (TableRules.bPrecacheTable && !bForceLoad)
			{
				UE_LOGF(LogDataRegistry, Verbose, "Synchronously precaching table %ls for source %ls, PreloadTable is %ls", *SourceTable.ToString(), *GetPathName(), *GetNameSafe(PreloadTable));
			}

			FoundTable = SourceTable.LoadSynchronous();
			if (!FoundTable)
			{
				if (!SourceTable.IsNull())
				{
					UE_LOGF(LogDataRegistry, Warning, "Force loading table %ls for source %ls failed! Source data is invalid and will be ignored.", *SourceTable.ToString(), *GetPathName());
				}
				bInvalidSourceTable = true;
			}
		}
	}

	if (FoundTable)
	{
		const UScriptStruct* ItemStruct = GetItemStruct();

		if (FoundTable->HasAnyFlags(RF_NeedLoad))
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, Preload table was not set, resave in editor!", *GetPathName());
			bInvalidSourceTable = true;
		}
		else if (!ItemStruct || !ItemStruct->IsChildOf(FRealCurve::StaticStruct()))
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, Curve tables only work with curve items!", *GetPathName());
			bInvalidSourceTable = true;
		}
		else if (ItemStruct->IsChildOf(FSimpleCurve::StaticStruct()) && FoundTable->GetCurveTableMode() != ECurveTableMode::SimpleCurves)
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, only simple curve tables are supported for SimpleCurve items, reimport with linear interpolation type or select different table", *GetPathName());
			bInvalidSourceTable = true;
		}
		else if (ItemStruct->IsChildOf(FRichCurve::StaticStruct()) && FoundTable->GetCurveTableMode() != ECurveTableMode::RichCurves)
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, only rich curve tables are supported for RichCurve items, reimport with cubic interpolation type or select different table", *GetPathName());
			bInvalidSourceTable = true;
		}
		else
		{
			CachedTable = FoundTable;
			bInvalidSourceTable = false;

			// Listen for changes like row 
			CachedTable->OnCurveTableChanged().AddUObject(this, &UDataRegistrySource_CurveTable::OnDataTableChanged);
		}
	}

	if (PreloadTable != CachedTable && TableRules.bPrecacheTable)
	{
		ensureMsgf(GIsEditor || !PreloadTable, TEXT("Switching a valid PreloadTable to a new table should only happen in the editor!"));
		PreloadTable = CachedTable;
	}

	LastAccessTime = UDataRegistry::GetCurrentTime();
}

void UDataRegistrySource_CurveTable::ClearCachedTable()
{
	// For soft refs this will null, for hard refs it will set to preload one
	CachedTable = PreloadTable;
}

void UDataRegistrySource_CurveTable::PostLoad()
{
	Super::PostLoad();

	SetCachedTable(false);
}

EDataRegistryAvailability UDataRegistrySource_CurveTable::GetSourceAvailability() const
{
	if (TableRules.bPrecacheTable)
	{
		return EDataRegistryAvailability::PreCached;
	}
	else
	{
		return EDataRegistryAvailability::LocalAsset;
	}
}

EDataRegistryAvailability UDataRegistrySource_CurveTable::GetItemAvailability(const FName& ResolvedName, const uint8** PrecachedDataPtr) const
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	if (CachedTable)
	{
		FRealCurve* FoundCurve = CachedTable->FindCurveUnchecked(ResolvedName);

		if (FoundCurve)
		{
			if (TableRules.bPrecacheTable)
			{
				// Return struct if found
				if (PrecachedDataPtr)
				{
					*PrecachedDataPtr = (const uint8 *)FoundCurve;
				}

				return EDataRegistryAvailability::PreCached;
			}
			else
			{
				return EDataRegistryAvailability::LocalAsset;
			}
		}
		else
		{
			return EDataRegistryAvailability::DoesNotExist;
		}
	}
	else if (bInvalidSourceTable)
	{
		return EDataRegistryAvailability::DoesNotExist;
	}

	return EDataRegistryAvailability::Unknown;
}

void UDataRegistrySource_CurveTable::GetResolvedNames(TArray<FName>& Names) const
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	if (!CachedTable && GIsEditor)
	{
		// Force load in editor
		const_cast<UDataRegistrySource_CurveTable*>(this)->SetCachedTable(true);
	}

	if (CachedTable)
	{
		CachedTable->GetRowMap().GetKeys(Names);
	}
}

void UDataRegistrySource_CurveTable::ResetRuntimeState()
{
	ClearCachedTable();

	if (LoadingTableHandle.IsValid())
	{
		LoadingTableHandle->CancelHandle();
		LoadingTableHandle.Reset();
	}

	Super::ResetRuntimeState();
}

bool UDataRegistrySource_CurveTable::AcquireItem(FDataRegistrySourceAcquireRequest&& Request)
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	PendingAcquires.Add(Request);

	if (CachedTable)
	{
		// Tell it to go next frame
		FStreamableHandle::ExecuteDelegate(FStreamableDelegate::CreateUObject(this, &UDataRegistrySource_CurveTable::HandlePendingAcquires));
	}
	else if (!LoadingTableHandle.IsValid() || !LoadingTableHandle->IsActive())
	{
		// If already in progress, don't request again
		LoadingTableHandle = UAssetManager::Get().LoadAssetList({ SourceTable.ToSoftObjectPath() }, FStreamableDelegate::CreateUObject(this, &UDataRegistrySource_CurveTable::OnTableLoaded));
	}

	return true;
}

void UDataRegistrySource_CurveTable::TimerUpdate(float CurrentTime, float TimerUpdateFrequency)
{
	Super::TimerUpdate(CurrentTime, TimerUpdateFrequency);

	// If we have a valid keep seconds, see if it has expired and release cache if needed
	if (TableRules.CachedTableKeepSeconds >= 0 && !TableRules.bPrecacheTable && CachedTable)
	{
		if (CurrentTime - LastAccessTime > TableRules.CachedTableKeepSeconds)
		{
			ClearCachedTable();
		}
	}
}

FString UDataRegistrySource_CurveTable::GetDebugString() const
{
	const UDataRegistry* Registry = GetRegistry();
	if (!SourceTable.IsNull() && Registry)
	{
		return FString::Printf(TEXT("%s(%d)"), *SourceTable.GetAssetName(), Registry->GetSourceIndex(this));
	}
	return Super::GetDebugString();
}

FSoftObjectPath UDataRegistrySource_CurveTable::GetSourceAssetPath() const
{
	return SourceTable.ToSoftObjectPath();
}

bool UDataRegistrySource_CurveTable::Initialize()
{
	if (Super::Initialize())
	{
		// Add custom logic

		return true;
	}

	return false;
}

void UDataRegistrySource_CurveTable::HandlePendingAcquires()
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	// Iterate manually to deal with recursive adds
	int32 NumRequests = PendingAcquires.Num();
	for (int32 i = 0; i < NumRequests; i++)
	{
		// Make a copy in case array changes
		FDataRegistrySourceAcquireRequest Request = PendingAcquires[i];

		uint8 Sourceindex = 255;
		FName ResolvedName;
			
		if (Request.Lookup.GetEntry(Sourceindex, ResolvedName, Request.LookupIndex))
		{
			if (CachedTable)
			{
				const UScriptStruct* ItemStruct = GetItemStruct();
				if (ensure(ItemStruct && ItemStruct->GetStructureSize()))
				{
					FRealCurve* FoundCurve = CachedTable->FindCurveUnchecked(ResolvedName);

					if (FoundCurve)
					{
						// Allocate new copy of struct, will be handed off to cache
						uint8* ItemStructMemory = FCachedDataRegistryItem::AllocateItemMemory(ItemStruct);

						ItemStruct->CopyScriptStruct(ItemStructMemory, FoundCurve);

						HandleAcquireResult(Request, EDataRegistryAcquireStatus::InitialAcquireFinished, ItemStructMemory);
						continue;
					}
				}
			}
		}
		else
		{
			// Invalid request
		}
		
		// Acquire failed for some reason, report failure for each one
		HandleAcquireResult(Request, EDataRegistryAcquireStatus::AcquireError, nullptr);
		
	}
		
	PendingAcquires.RemoveAt(0, NumRequests);
}

void UDataRegistrySource_CurveTable::OnTableLoaded()
{
	// Set cache pointer than handle any pending requests
	LoadingTableHandle.Reset();

	SetCachedTable(false);

	// If we failed to set a cached table, the source data is invalid
	if (CachedTable == nullptr)
	{
		UE_LOGF(LogDataRegistry, Warning, "Loading table %ls for source %ls failed! Source data is invalid and will be ignored.", *SourceTable.ToString(), *GetPathName());
		bInvalidSourceTable = true;
	}

	HandlePendingAcquires();
}

void UDataRegistrySource_CurveTable::OnDataTableChanged()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		SetCachedTable(false);
	}
#endif

	if (IsInitialized())
	{
		GetRegistry()->InvalidateCacheVersion();
	}
}

#if WITH_EDITOR

void UDataRegistrySource_CurveTable::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Force load it to validate type on save
	SetCachedTable(true);
}

void UDataRegistrySource_CurveTable::EditorRefreshSource()
{
	SetCachedTable(false);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


UMetaDataRegistrySource_CurveTable::UMetaDataRegistrySource_CurveTable()
{
	CreatedSource = UDataRegistrySource_CurveTable::StaticClass();
	SearchRules.AssetBaseClass = UCurveTable::StaticClass();
}

TSubclassOf<UDataRegistrySource> UMetaDataRegistrySource_CurveTable::GetChildSourceClass() const
{
	return CreatedSource;
}

bool UMetaDataRegistrySource_CurveTable::SetDataForChild(FName SourceId, UDataRegistrySource* ChildSource)
{
	UDataRegistrySource_CurveTable* ChildCurveTable = Cast<UDataRegistrySource_CurveTable>(ChildSource);
	if (ensure(ChildCurveTable))
	{
		TSoftObjectPtr<UCurveTable> NewTable = TSoftObjectPtr<UCurveTable>(FSoftObjectPath(SourceId.ToString()));
		ChildCurveTable->SetSourceTable(NewTable, TableRules);
		return true;
	}
	return false;
}

bool UMetaDataRegistrySource_CurveTable::DoesAssetPassFilter(const FAssetData& AssetData, bool bNewRegisteredAsset)
{
	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();
	
	// Call into parent to check search rules if needed	
	if (bNewRegisteredAsset)
	{
		FAssetManagerSearchRules ModifiedRules = SearchRules;

		if (Settings->CanIgnoreMissingAssetData())
		{
			// Drop the class check, only do basic path validation
			ModifiedRules.AssetBaseClass = nullptr;
		}

		if (!UAssetManager::Get().DoesAssetMatchSearchRules(AssetData, ModifiedRules))
		{
			return false;
		}
	}

	// TODO no good way to validate unloaded curvetables
	return true;
}




