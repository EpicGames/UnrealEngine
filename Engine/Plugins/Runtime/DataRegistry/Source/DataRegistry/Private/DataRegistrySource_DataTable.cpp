// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistrySource_DataTable.h"

#include "AssetRegistry/AssetData.h"
#include "DataRegistryTypesPrivate.h"
#include "DataRegistrySettings.h"
#include "Engine/AssetManager.h"
#include "Engine/DataTable.h"
#include "UObject/CoreRedirects.h"
#include "Engine/StreamableManager.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectThreadContext.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistrySource_DataTable)

void UDataRegistrySource_DataTable::SetSourceTable(const TSoftObjectPtr<UDataTable>& InSourceTable, const FDataRegistrySource_DataTableRules& InTableRules)
{
	if (ensure(IsTransientSource() || GIsEditor))
	{
		SourceTable = InSourceTable;
		TableRules = InTableRules;
		SetCachedTable(false);
	}
}

void UDataRegistrySource_DataTable::SetCachedTable(bool bForceLoad /*= false*/)
{
	if (CachedTable)
	{
		CachedTable->OnDataTableChanged().RemoveAll(this);
	}

	CachedTable = nullptr;
	UDataTable* FoundTable = SourceTable.Get();

	if (!FoundTable && (bForceLoad || TableRules.bPrecacheTable))
	{
		if (FUObjectThreadContext::Get().IsRoutingPostLoad)
		{
			// It is not safe to try and preload during a postload, this should only happen if PreloadTable is null
			UE_LOGF(LogDataRegistry, Log, "Failed to preload table %ls for source %ls, PreloadTable is %ls", *SourceTable.ToString(), *GetPathName(), *GetNameSafe(PreloadTable));
		}
		else
		{
			if (TableRules.bPrecacheTable && !bForceLoad)
			{
				UE_LOGF(LogDataRegistry, Verbose, "Synchronously precaching table %ls for source %ls, PreloadTable is %ls", *SourceTable.ToString(), *GetPathName(), *GetNameSafe(PreloadTable));
			}

			FoundTable = SourceTable.LoadSynchronous();
			if (!FoundTable)
			{
				if (!SourceTable.IsNull())
				{
					UE_LOGF(LogDataRegistry, Warning, "Force loading table %ls for source %ls failed! Source data is invalid and will be ignored.", *SourceTable.ToString(), *GetPathName());
				}
				bInvalidSourceTable = true;
			}
		}
	}

	if (FoundTable)
	{
		const UScriptStruct* ItemStruct = GetItemStruct();
		const UScriptStruct* RowStruct = FoundTable->GetRowStruct();

		if (FoundTable->HasAnyFlags(RF_NeedLoad))
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, Preload table was not set, resave in editor!", *GetPathName());
			bInvalidSourceTable = true;
		}
		else if(!ItemStruct || !RowStruct)
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, Table %ls or registry is invalid!", *GetPathName(), *FoundTable->GetPathName());
			bInvalidSourceTable = true;
		}
		else if (!RowStruct->IsChildOf(ItemStruct))
		{
			UE_LOGF(LogDataRegistry, Error, "Cannot initialize DataRegistry source %ls, Table %ls row type %ls does not match item type %ls", *GetPathName(), *FoundTable->GetPathName(), *RowStruct->GetName(), *ItemStruct->GetName());
			bInvalidSourceTable = true;
		}
		else
		{
			CachedTable = FoundTable;
			bInvalidSourceTable = false;

			// Listen for changes like row 
			CachedTable->OnDataTableChanged().AddUObject(this, &UDataRegistrySource_DataTable::OnDataTableChanged);
		}
	}

	if (PreloadTable != CachedTable && TableRules.bPrecacheTable)
	{
		ensureMsgf(GIsEditor || !PreloadTable, TEXT("Switching a valid PreloadTable to a new table should only happen in the editor!"));
		PreloadTable = CachedTable;
	}

	LastAccessTime = UDataRegistry::GetCurrentTime();
}

void UDataRegistrySource_DataTable::ClearCachedTable()
{
	// For soft refs this will null, for hard refs it will set to preload one
	CachedTable = PreloadTable;
}

void UDataRegistrySource_DataTable::PostLoad()
{
	Super::PostLoad();

	SetCachedTable(false);
}

EDataRegistryAvailability UDataRegistrySource_DataTable::GetSourceAvailability() const
{
	if (TableRules.bPrecacheTable)
	{
		return EDataRegistryAvailability::PreCached;
	}
	else
	{
		return EDataRegistryAvailability::LocalAsset;
	}
}

EDataRegistryAvailability UDataRegistrySource_DataTable::GetItemAvailability(const FName& ResolvedName, const uint8** PrecachedDataPtr) const
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	if (CachedTable)
	{
		uint8* FoundRow = CachedTable->FindRowUnchecked(ResolvedName);

		if (FoundRow)
		{
			if (TableRules.bPrecacheTable)
			{
				// Return struct if found
				if (PrecachedDataPtr)
				{
					*PrecachedDataPtr = FoundRow;
				}

				return EDataRegistryAvailability::PreCached;
			}
			else
			{
				return EDataRegistryAvailability::LocalAsset;
			}
		}
		else
		{
			return EDataRegistryAvailability::DoesNotExist;
		}
	}
	else if (bInvalidSourceTable)
	{
		return EDataRegistryAvailability::DoesNotExist;
	}

	return EDataRegistryAvailability::Unknown;
}

void UDataRegistrySource_DataTable::GetResolvedNames(TArray<FName>& Names) const
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	if (!CachedTable && GIsEditor)
	{
		// Force load in editor
		const_cast<UDataRegistrySource_DataTable*>(this)->SetCachedTable(true);
	}

	if (CachedTable)
	{
		Names = CachedTable->GetRowNames();
	}
}

void UDataRegistrySource_DataTable::ResetRuntimeState()
{
	ClearCachedTable();

	if (LoadingTableHandle.IsValid())
	{
		LoadingTableHandle->CancelHandle();
		LoadingTableHandle.Reset();
	}

	Super::ResetRuntimeState();
}

bool UDataRegistrySource_DataTable::AcquireItem(FDataRegistrySourceAcquireRequest&& Request)
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	PendingAcquires.Add(Request);

	if (CachedTable)
	{
		// Tell it to go next frame
		FStreamableHandle::ExecuteDelegate(FStreamableDelegate::CreateUObject(this, &UDataRegistrySource_DataTable::HandlePendingAcquires));
	}
	else if (!LoadingTableHandle.IsValid() || !LoadingTableHandle->IsActive())
	{
		// If already in progress, don't request again
		LoadingTableHandle = UAssetManager::Get().LoadAssetList({ SourceTable.ToSoftObjectPath() }, FStreamableDelegate::CreateUObject(this, &UDataRegistrySource_DataTable::OnTableLoaded));
	}

	return true;
}

void UDataRegistrySource_DataTable::TimerUpdate(float CurrentTime, float TimerUpdateFrequency)
{
	Super::TimerUpdate(CurrentTime, TimerUpdateFrequency);

	// If we have a valid keep seconds, see if it has expired and release cache if needed
	if (TableRules.CachedTableKeepSeconds >= 0 && !TableRules.bPrecacheTable && CachedTable)
	{
		if (CurrentTime - LastAccessTime > TableRules.CachedTableKeepSeconds)
		{
			ClearCachedTable();
		}
	}
}

FString UDataRegistrySource_DataTable::GetDebugString() const
{
	const UDataRegistry* Registry = GetRegistry();
	if (!SourceTable.IsNull() && Registry)
	{
		return FString::Printf(TEXT("%s(%d)"), *SourceTable.GetAssetName(), Registry->GetSourceIndex(this));
	}
	return Super::GetDebugString();
}

FSoftObjectPath UDataRegistrySource_DataTable::GetSourceAssetPath() const
{
	return SourceTable.ToSoftObjectPath();
}

bool UDataRegistrySource_DataTable::Initialize()
{
	if (Super::Initialize())
	{
		// Add custom logic

		return true;
	}

	return false;
}

void UDataRegistrySource_DataTable::HandlePendingAcquires()
{
	LastAccessTime = UDataRegistry::GetCurrentTime();

	// Iterate manually to deal with recursive adds
	int32 NumRequests = PendingAcquires.Num();
	for (int32 i = 0; i < NumRequests; i++)
	{
		// Make a copy in case array changes
		FDataRegistrySourceAcquireRequest Request = PendingAcquires[i];

		uint8 Sourceindex = 255;
		FName ResolvedName;
			
		if (Request.Lookup.GetEntry(Sourceindex, ResolvedName, Request.LookupIndex))
		{
			if (CachedTable)
			{
				const UScriptStruct* ItemStruct = GetItemStruct();
				if (ensure(ItemStruct && ItemStruct->GetStructureSize()))
				{
					uint8* FoundRow = CachedTable->FindRowUnchecked(ResolvedName);

					if (FoundRow)
					{
						// Allocate new copy of struct, will be handed off to cache
						uint8* ItemStructMemory = FCachedDataRegistryItem::AllocateItemMemory(ItemStruct);

						ItemStruct->CopyScriptStruct(ItemStructMemory, FoundRow);

						HandleAcquireResult(Request, EDataRegistryAcquireStatus::InitialAcquireFinished, ItemStructMemory);
						continue;
					}
				}
			}
		}
		else
		{
			// Invalid request
		}
		
		// Acquire failed for some reason, report failure for each one
		HandleAcquireResult(Request, EDataRegistryAcquireStatus::AcquireError, nullptr);
		
	}
		
	PendingAcquires.RemoveAt(0, NumRequests);
}

void UDataRegistrySource_DataTable::OnTableLoaded()
{
	// Set cache pointer than handle any pending requests
	LoadingTableHandle.Reset();

	SetCachedTable(false);

	// If we failed to set a cached table, the source data is invalid
	if (CachedTable == nullptr)
	{
		UE_LOGF(LogDataRegistry, Warning, "Loading table %ls for source %ls failed! Source data is invalid and will be ignored.", *SourceTable.ToString(), *GetPathName());
		bInvalidSourceTable = true;
	}

	HandlePendingAcquires();
}

void UDataRegistrySource_DataTable::OnDataTableChanged()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		SetCachedTable(false);
	}
#endif

	if (IsInitialized())
	{
		GetRegistry()->InvalidateCacheVersion();
	}
}

#if WITH_EDITOR

void UDataRegistrySource_DataTable::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Force load it to validate type on save
	SetCachedTable(true);
}

void UDataRegistrySource_DataTable::EditorRefreshSource()
{
	SetCachedTable(false);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


UMetaDataRegistrySource_DataTable::UMetaDataRegistrySource_DataTable()
{
	CreatedSource = UDataRegistrySource_DataTable::StaticClass();
	SearchRules.AssetBaseClass = UDataTable::StaticClass();
}

TSubclassOf<UDataRegistrySource> UMetaDataRegistrySource_DataTable::GetChildSourceClass() const
{
	return CreatedSource;
}

bool UMetaDataRegistrySource_DataTable::SetDataForChild(FName SourceId, UDataRegistrySource* ChildSource)
{
	UDataRegistrySource_DataTable* ChildDataTable = Cast<UDataRegistrySource_DataTable>(ChildSource);
	if (ensure(ChildDataTable))
	{
		TSoftObjectPtr<UDataTable> NewTable = TSoftObjectPtr<UDataTable>(FSoftObjectPath(SourceId.ToString()));
		ChildDataTable->SetSourceTable(NewTable, TableRules);
		return true;
	}
	return false;
}

bool UMetaDataRegistrySource_DataTable::DoesAssetPassFilter(const FAssetData& AssetData, bool bNewRegisteredAsset)
{
	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();
	
	// Call into parent to check search rules if needed	
	if (bNewRegisteredAsset)
	{
		FAssetManagerSearchRules ModifiedRules = SearchRules;

		if (Settings->CanIgnoreMissingAssetData())
		{
			// Drop the class check, only do basic path validation
			ModifiedRules.AssetBaseClass = nullptr;
		}

		if (!UAssetManager::Get().DoesAssetMatchSearchRules(AssetData, ModifiedRules))
		{
			return false;
		}
	}

	static const FName RowStructureTagName("RowStructure");
	FString RowStructureString;
	if (AssetData.GetTagValue(RowStructureTagName, RowStructureString))
	{
		if (const UScriptStruct* ItemStruct = GetItemStruct())
		{
			if (RowStructureString == ItemStruct->GetName() || RowStructureString == ItemStruct->GetStructPathName().ToString())
			{
				return true;
			}
			else
			{
				// TODO no 100% way to check for inherited row structs, but BP types can't inherit anyway
				const UScriptStruct* RowStruct = FindFirstObject<UScriptStruct>(*RowStructureString, EFindFirstObjectOptions::ExactClass);

				// Check if the row struct is a child of the item struct
				if (RowStruct != nullptr)
				{
					if (RowStruct->IsChildOf(ItemStruct))
					{
						return true;
					}	
				}
				// Otherwise check if the row struct has been redirected to the item struct
				else
				{
					FName RowStructureName = FName(FPackageName::ObjectPathToObjectName(RowStructureString)); // RobM this should use path names as well
				
					TArray<FCoreRedirectObjectName> PreviousNames;
					FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Struct, ItemStruct->GetPathName(), PreviousNames);
				
					return PreviousNames.ContainsByPredicate([&RowStructureName](const FCoreRedirectObjectName& PreviousName){ return PreviousName.ObjectName == RowStructureName; });
				}
			}
		}
	}
	else if (Settings->CanIgnoreMissingAssetData())
	{
		// Row may have been stripped out, so assume it is valid
		return true;
	}

	return false;
}


