// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistrySubsystem.h"
#include "UObject/UObjectIterator.h"
#include "Engine/AssetManagerTypes.h"
#include "GameplayTagsManager.h"
#include "Engine/AssetManager.h"
#include "DataRegistrySettings.h"
#include "Curves/RealCurve.h"
#include "Engine/StreamableManager.h"
#include "Misc/CoreDelegates.h"
#include "UObject/EnumProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistrySubsystem)

#if WITH_EDITOR
#include "Editor.h"
#else
#include "AssetRegistry/AssetData.h"
#include "Engine/Engine.h"
#endif

TObjectPtr<UDataRegistrySubsystem> UDataRegistrySubsystem::SingletonSubSystem = nullptr;

UDataRegistrySubsystem* UDataRegistrySubsystem::Get()
{
	// GetEngineSubsystem is fairly slow and once SingletonSubSystem is valid it will stay valid until shutdown
	return SingletonSubSystem;
}

//static bool GetCachedItemBP(FDataRegistryId ItemId, UPARAM(ref) FTableRowBase& OutItem);
DEFINE_FUNCTION(UDataRegistrySubsystem::execGetCachedItemBP)
{
	P_GET_STRUCT(FDataRegistryId, ItemId);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* OutItemDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* OutItemProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	const uint8* CacheData = nullptr;
	const UScriptStruct* CacheStruct = nullptr;
	FDataRegistryCacheGetResult CacheResult;
	EDataRegistrySubsystemGetItemResult OutResult = EDataRegistrySubsystemGetItemResult::NotFound;

	if (OutItemProp && OutItemDataPtr && SubSystem->IsConfigEnabled(true))
	{
		P_NATIVE_BEGIN;
		CacheResult = SubSystem->GetCachedItemRaw(CacheData, CacheStruct, ItemId);

		if (CacheResult && CacheStruct && CacheData)
		{
			UScriptStruct* OutputStruct = OutItemProp->Struct;

			const bool bCompatible = (OutputStruct == CacheStruct) ||
				(OutputStruct->IsChildOf(CacheStruct) && FStructUtils::TheSameLayout(OutputStruct, CacheStruct));
			
			if (bCompatible)
			{
				OutResult = EDataRegistrySubsystemGetItemResult::Found;
				CacheStruct->CopyScriptStruct(OutItemDataPtr, CacheData);
			}
		}
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = (OutResult == EDataRegistrySubsystemGetItemResult::Found);
}

//static bool FindCachedItemBP(FDataRegistryId ItemId, EDataRegistrySubsystemGetItemResult& OutResult, FTableRowBase& OutItem) {}
DEFINE_FUNCTION(UDataRegistrySubsystem::execFindCachedItemBP)
{
	P_GET_STRUCT(FDataRegistryId, ItemId);
	P_GET_ENUM_REF(EDataRegistrySubsystemGetItemResult, OutResult)

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* OutItemDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* OutItemProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	const uint8* CacheData = nullptr;
	const UScriptStruct* CacheStruct = nullptr;
	FDataRegistryCacheGetResult CacheResult;
	OutResult = EDataRegistrySubsystemGetItemResult::NotFound;

	if (OutItemProp && OutItemDataPtr && SubSystem->IsConfigEnabled(true))
	{
		P_NATIVE_BEGIN;
		CacheResult = SubSystem->GetCachedItemRaw(CacheData, CacheStruct, ItemId);

		if (CacheResult && CacheStruct && CacheData)
		{
			UScriptStruct* OutputStruct = OutItemProp->Struct;

			const bool bCompatible = (OutputStruct == CacheStruct) ||
				(OutputStruct->IsChildOf(CacheStruct) && FStructUtils::TheSameLayout(OutputStruct, CacheStruct));

			if (bCompatible)
			{
				OutResult = EDataRegistrySubsystemGetItemResult::Found;
				CacheStruct->CopyScriptStruct(OutItemDataPtr, CacheData);
			}
		}
		P_NATIVE_END;
	}
}


//static void FindCachedItemFromLookupBP(FDataRegistryId ItemId, const FDataRegistryLookup& ResolvedLookup, EDataRegistrySubsystemGetItemResult& OutResult, FTableRowBase& OutItem) {}
DEFINE_FUNCTION(UDataRegistrySubsystem::execFindCachedItemFromLookupBP)
{
	P_GET_STRUCT(FDataRegistryId, ItemId);
	P_GET_STRUCT_REF(FDataRegistryLookup, ResolvedLookup);
	P_GET_ENUM_REF(EDataRegistrySubsystemGetItemResult, OutResult)

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* OutItemDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* OutItemProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	const uint8* CacheData = nullptr;
	const UScriptStruct* CacheStruct = nullptr;
	FDataRegistryCacheGetResult CacheResult;
	OutResult = EDataRegistrySubsystemGetItemResult::NotFound;

	if (OutItemProp && OutItemDataPtr && SubSystem->IsConfigEnabled(true))
	{
		P_NATIVE_BEGIN;
		CacheResult = SubSystem->GetCachedItemRawFromLookup(CacheData, CacheStruct, ItemId, ResolvedLookup);

		if (CacheResult && CacheStruct && CacheData)
		{
			UScriptStruct* OutputStruct = OutItemProp->Struct;

			const bool bCompatible = (OutputStruct == CacheStruct) ||
				(OutputStruct->IsChildOf(CacheStruct) && FStructUtils::TheSameLayout(OutputStruct, CacheStruct));

			if (bCompatible)
			{
				OutResult = EDataRegistrySubsystemGetItemResult::Found;
				CacheStruct->CopyScriptStruct(OutItemDataPtr, CacheData);
			}
		}
		P_NATIVE_END;
	}
}

//static bool GetCachedItemFromLookupBP(FDataRegistryId ItemId, const FDataRegistryLookup& ResolvedLookup, UPARAM(ref) FTableRowBase& OutItem) { return false; }
DEFINE_FUNCTION(UDataRegistrySubsystem::execGetCachedItemFromLookupBP)
{
	P_GET_STRUCT(FDataRegistryId, ItemId);
	P_GET_STRUCT_REF(FDataRegistryLookup, ResolvedLookup);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* OutItemDataPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* OutItemProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	P_FINISH;

	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	const uint8* CacheData = nullptr;
	const UScriptStruct* CacheStruct = nullptr;
	FDataRegistryCacheGetResult CacheResult;

	if (OutItemProp && OutItemDataPtr && SubSystem->IsConfigEnabled(true))
	{
		P_NATIVE_BEGIN;
		CacheResult = SubSystem->GetCachedItemRawFromLookup(CacheData, CacheStruct, ItemId, ResolvedLookup);

		if (CacheResult && CacheStruct && CacheData)
		{
			UScriptStruct* OutputStruct = OutItemProp->Struct;

			const bool bCompatible = (OutputStruct == CacheStruct) ||
				(OutputStruct->IsChildOf(CacheStruct) && FStructUtils::TheSameLayout(OutputStruct, CacheStruct));

			if (bCompatible)
			{
				CacheStruct->CopyScriptStruct(OutItemDataPtr, CacheData);
			}
		}
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = !!CacheResult;
}


UDataRegistry* UDataRegistrySubsystem::GetRegistryForType(FName RegistryType) const
{
	FlushDeferredRegistryLoad(false);

	auto* FoundRegistry = RegistryMap.Find(RegistryType);

	if (FoundRegistry && *FoundRegistry)
	{
		return *FoundRegistry;
	}

	return nullptr;
}

FText UDataRegistrySubsystem::GetDisplayTextForId(FDataRegistryId ItemId) const
{
	UDataRegistry* DataRegistry = GetRegistryForType(ItemId.RegistryType);
	if (DataRegistry)
	{
		// If tag prefix is redundant, strip it
		const FDataRegistryIdFormat& IdFormat = DataRegistry->GetIdFormat();

		if (IdFormat.BaseGameplayTag.GetTagName() == ItemId.RegistryType.GetName())
		{
			return FText::FromName(ItemId.ItemName);
		}
	}

	return ItemId.ToText();
}

void UDataRegistrySubsystem::GetAllRegistries(TArray<UDataRegistry*>& AllRegistries, bool bSortByType) const
{
	AllRegistries.Reset();

	FlushDeferredRegistryLoad(false);

	for (const FRegistryMapPair& RegistryPair : RegistryMap)
	{
		UDataRegistry* Registry = RegistryPair.Value;
		if (Registry)
		{
			AllRegistries.Add(Registry);
		}
	}

	if (bSortByType)
	{
		AllRegistries.Sort([](const UDataRegistry& LHS, const UDataRegistry& RHS) { return LHS.GetRegistryType().LexicalLess(RHS.GetRegistryType()); });
	}
}

void UDataRegistrySubsystem::RefreshRegistryMap()
{
	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();

	auto OldMap = RegistryMap;
	RegistryMap.Reset();

	for (TObjectIterator<UDataRegistry> RegistryIterator; RegistryIterator; ++RegistryIterator)
	{
		UDataRegistry* Registry = *RegistryIterator;
		FString ObjectPathString = Registry->GetPathName();
		FSoftObjectPath ObjectSoftPath = FSoftObjectPath(Registry);

		if (!Settings->bInitializeAllLoadedRegistries)
		{
			// Check it's one of the scanned directories
			bool bFoundPath = false;
			for (const FString& ScanPath : AssetScanPaths)
			{
				if (ObjectPathString.StartsWith(ScanPath))
				{
					bFoundPath = true;
					break;
				}
			}

			if (!bFoundPath)
			{
				continue;
			}
		}

		// Always check exclusion paths
		if (RegistryPathsToIgnore.Contains(ObjectSoftPath))
		{
			continue;
		}

		RegistryMap.Add(Registry->GetRegistryType(), Registry);
		
		// Apply pending map before we initialize
		if (!Registry->IsInitialized())
		{
			ApplyPreregisterMap(Registry);
		}
	}

	// Deinitialize anything that is no longer valid
	for (const FRegistryMapPair& RegistryPair : OldMap)
	{
		UDataRegistry* Registry = RegistryPair.Value;
		if (Registry && Registry->IsInitialized() && !RegistryMap.Contains(RegistryPair.Key))
		{
			Registry->Deinitialize();
		}
	}
}

void UDataRegistrySubsystem::LoadAllRegistries()
{
	SCOPED_BOOT_TIMING("UDataRegistrySubsystem::LoadAllRegistries");

	if (!IsConfigEnabled())
	{
		// Don't do anything if not enabled
		return;
	}

	OnPreLoadAllDataRegistries().Broadcast();

	UAssetManager& AssetManager = UAssetManager::Get();
	IAssetRegistry& AssetRegistry = AssetManager.GetAssetRegistry();

	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();

	AssetScanPaths.Reset();
	for (const FDirectoryPath& PathRef : Settings->DirectoriesToScan)
	{
		if (!PathRef.Path.IsEmpty())
		{
			AssetScanPaths.AddUnique(UAssetManager::GetNormalizedPackagePath(PathRef.Path, false));
		}
	}

	for (const FSoftObjectPath& ObjectPath : RegistryPathsToLoad)
	{
		AssetScanPaths.Add(ObjectPath.ToString());
	}

	FAssetManagerSearchRules Rules;
	Rules.AssetScanPaths = AssetScanPaths;
	Rules.AssetBaseClass = UDataRegistry::StaticClass();

	bool bExpandedVirtual = AssetManager.ExpandVirtualPaths(Rules.AssetScanPaths);
	Rules.bSkipVirtualPathExpansion = true;

	if (bExpandedVirtual)
	{
		// Handling this case properly would require some with modular feature code
		UE_LOGF(LogDataRegistry, Error, "DataRegistries do not refresh in virtual asset search roots, use LoadRegistryPath instead");
	}

	TArray<FAssetData> AssetDataList;
	AssetManager.SearchAssetRegistryPaths(AssetDataList, Rules);

	TArray<FSoftObjectPath> PathsToLoad;

	// Now add to map or update as needed
	for (FAssetData& Data : AssetDataList)
	{
		PathsToLoad.AddUnique(AssetManager.GetAssetPathForData(Data));
	}

	if (PathsToLoad.Num() > 0)
	{
		// Do as one async bulk load, faster in cooked builds
		TSharedPtr<FStreamableHandle> LoadHandle = AssetManager.LoadAssetList(MoveTemp(PathsToLoad));

		if (LoadHandle.IsValid())
		{
			LoadHandle->WaitUntilComplete();
		}
	}

	RefreshRegistryMap();
	InitializeAllRegistries();
}

bool UDataRegistrySubsystem::AreRegistriesInitialized() const
{
	return bFullyInitialized;
}

bool UDataRegistrySubsystem::IsConfigEnabled(bool bWarnIfNotEnabled /*= false*/) const
{
	if (bFullyInitialized)
	{
		return true;
	}

	if (RegistryPathsToLoad.Num() > 0)
	{
		return true;
	}

	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();
	if (Settings->DirectoriesToScan.Num() == 0 && !Settings->bInitializeAllLoadedRegistries)
	{
		if (bWarnIfNotEnabled)
		{
			UE_LOGF(LogDataRegistry, Warning, "DataRegistry functions are not enabled, to fix set scan options in the DataRegistry settings.");
		}
		return false;
	}

	return true;
}

bool UDataRegistrySubsystem::IsReadyForInitialization() const
{
#if WITH_EDITOR
	return bReadyForInitialization || bLoadAllRegistriesOnNextPIE;
#else
	return bReadyForInitialization;
#endif
}

FDataRegistrySubsystemInitializedCallback& UDataRegistrySubsystem::OnSubsystemInitialized()
{
	return OnSubsystemInitializedCallback;
}

FPreLoadAllDataRegistriesCallback& UDataRegistrySubsystem::OnPreLoadAllDataRegistries()
{
	return OnPreLoadAllDataRegistriesCallback;
}

void UDataRegistrySubsystem::InitializeAllRegistries(bool bResetIfInitialized)
{
	for (const FRegistryMapPair& RegistryPair : RegistryMap)
	{
		UDataRegistry* Registry = RegistryPair.Value;
		if (Registry && !Registry->IsInitialized())
		{
			Registry->Initialize();
		}
		else if (Registry && bResetIfInitialized)
		{
			Registry->ResetRuntimeState();
		}
	}

	bFullyInitialized = true;

	OnSubsystemInitialized().Broadcast();
}

void UDataRegistrySubsystem::DeinitializeAllRegistries()
{
	for (const FRegistryMapPair& RegistryPair : RegistryMap)
	{
		UDataRegistry* Registry = RegistryPair.Value;
		if (Registry)
		{
			Registry->Deinitialize();
		}
	}

	bFullyInitialized = false;
}

bool UDataRegistrySubsystem::LoadRegistryPath(const FSoftObjectPath& RegistryAssetPath)
{
	if (RegistryPathsToLoad.AddUnique(RegistryAssetPath) != INDEX_NONE)
	{
		RegistryPathsToIgnore.Remove(RegistryAssetPath);
		if (bReadyForInitialization)
		{
			// Need to make sure it's in memory
			RegistryAssetPath.TryLoad();

			// If we're past initialization, add it to the path list so it doesn't get filtered out
			AssetScanPaths.AddUnique(RegistryAssetPath.ToString());

			RefreshRegistryMap();
			InitializeAllRegistries(false);
		}

		return true;
	}
	return false;
}

bool UDataRegistrySubsystem::IgnoreRegistryPath(const FSoftObjectPath& RegistryAssetPath)
{
	if (RegistryPathsToIgnore.AddUnique(RegistryAssetPath) != INDEX_NONE)
	{
		RegistryPathsToLoad.Remove(RegistryAssetPath);
		if (bReadyForInitialization)
		{
			// Remove if active
			RefreshRegistryMap();
		}
		
		return true;
	}
	return false;
}

void UDataRegistrySubsystem::ResetRuntimeState()
{
	for (const FRegistryMapPair& RegistryPair : RegistryMap)
	{
		UDataRegistry* Registry = RegistryPair.Value;
		if (Registry && Registry->IsInitialized())
		{
			Registry->ResetRuntimeState();
		}
	}
}

void UDataRegistrySubsystem::ReinitializeFromConfig()
{
	if (bReadyForInitialization)
	{
		LoadAllRegistries();
	}
}

bool UDataRegistrySubsystem::RegisterSpecificAsset(FDataRegistryType RegistryType, FAssetData& AssetData, int32 AssetPriority /*= 0*/)
{
	if (!IsConfigEnabled(true))
	{
		return false;
	}

	// Update priority in pending list if found
	TArray<FPreregisterAsset>* FoundPreregister = PreregisterAssetMap.Find(RegistryType);

	if (FoundPreregister)
	{
		FSoftObjectPath AssetPath = AssetData.ToSoftObjectPath();
		for (int32 i = 0; i < FoundPreregister->Num(); i++)
		{
			if (AssetPath == (*FoundPreregister)[i].Key)
			{
				(*FoundPreregister)[i].Value = AssetPriority;
				break;
			}
		}
	}

	if (!RegistryType.IsValid())
	{
		FlushDeferredRegistryLoad(false);

		bool bMadeChange = false;
		for (const FRegistryMapPair& RegistryPair : RegistryMap)
		{
			UDataRegistry* Registry = RegistryPair.Value;
			if (Registry)
			{
				bMadeChange |= Registry->DidRegisterAssetResultCauseChange(Registry->RegisterSpecificAsset(AssetData, AssetPriority));
			}
		}
		return bMadeChange;
	}

	UDataRegistry* FoundRegistry = GetRegistryForType(RegistryType);
	if (FoundRegistry)
	{
		return FoundRegistry->DidRegisterAssetResultCauseChange(FoundRegistry->RegisterSpecificAsset(AssetData, AssetPriority));
	}

	return false;
}

bool UDataRegistrySubsystem::UnregisterSpecificAsset(FDataRegistryType RegistryType, const FSoftObjectPath& AssetPath)
{
	if (!IsConfigEnabled(true))
	{
		return false;
	}

	// First take out of pending list
	TArray<FPreregisterAsset>* FoundPreregister = PreregisterAssetMap.Find(RegistryType);

	if (FoundPreregister)
	{
		for (int32 i = 0; i < FoundPreregister->Num(); i++)
		{
			if (AssetPath == (*FoundPreregister)[i].Key)
			{
				FoundPreregister->RemoveAt(i);
				break;
			}
		}
	}

	bool bMadeChange = false;

	if (!RegistryType.IsValid())
	{
		for (const FRegistryMapPair& RegistryPair : RegistryMap)
		{
			UDataRegistry* Registry = RegistryPair.Value;
			if (Registry)
			{
				bMadeChange |= Registry->UnregisterSpecificAsset(AssetPath);
			}
		}
	}
	else
	{
		UDataRegistry* FoundRegistry = GetRegistryForType(RegistryType);
		if (FoundRegistry)
		{
			bMadeChange = FoundRegistry->UnregisterSpecificAsset(AssetPath);
		}
	}

	if (!bMadeChange)
	{
		// Data registry may have already unregistered, look at the other ones in memory that are not CDOs
		for (TObjectIterator<UDataRegistry> RegistryIterator; RegistryIterator; ++RegistryIterator)
		{
			UDataRegistry* FoundRegistry = *RegistryIterator;
			if (!FoundRegistry->IsInitialized() && (!RegistryType.IsValid() || FoundRegistry->GetRegistryType() == RegistryType))
			{
				bMadeChange = FoundRegistry->UnregisterSpecificAsset(AssetPath);
			}
		}
	}

	return bMadeChange;
}

int32 UDataRegistrySubsystem::UnregisterAssetsWithPriority(FDataRegistryType RegistryType, int32 AssetPriority)
{
	int32 NumberUnregistered = 0;
	if (!IsConfigEnabled(true))
	{
		return NumberUnregistered;
	}

	// First take out of pending list
	TArray<FPreregisterAsset>* FoundPreregister = PreregisterAssetMap.Find(RegistryType);

	if (FoundPreregister)
	{
		for (int32 i = 0; i < FoundPreregister->Num(); i++)
		{
			if (AssetPriority == (*FoundPreregister)[i].Value)
			{
				FoundPreregister->RemoveAt(i);
				i--;
			}
		}
	}

	if (!RegistryType.IsValid())
	{
		for (const FRegistryMapPair& RegistryPair : RegistryMap)
		{
			UDataRegistry* Registry = RegistryPair.Value;
			if (Registry)
			{
				NumberUnregistered += Registry->UnregisterAssetsWithPriority(AssetPriority);
			}
		}
	}
	else
	{
		UDataRegistry* FoundRegistry = GetRegistryForType(RegistryType);
		if (FoundRegistry)
		{
			NumberUnregistered += FoundRegistry->UnregisterAssetsWithPriority(AssetPriority);
		}
	}

	return NumberUnregistered;
}

void UDataRegistrySubsystem::PreregisterSpecificAssets(const TMap<FDataRegistryType, TArray<FSoftObjectPath>>& AssetMap, int32 AssetPriority)
{
	if (!IsConfigEnabled(true))
	{
		return;
	}

	TSet<FName> ChangedTypeSet;
	for (const TPair<FDataRegistryType, TArray<FSoftObjectPath>>& TypePair : AssetMap)
	{
		// Update map, then apply if already exists
		TArray<FPreregisterAsset>& FoundPreregister = PreregisterAssetMap.FindOrAdd(TypePair.Key);

		for (const FSoftObjectPath& PathToAdd : TypePair.Value)
		{
			bool bFoundAsset = false;
			for (int32 i = 0; i < FoundPreregister.Num(); i++)
			{
				if (PathToAdd == FoundPreregister[i].Key)
				{
					bFoundAsset = true;
					if (AssetPriority != FoundPreregister[i].Value)
					{
						// Update value
						FoundPreregister[i].Value = AssetPriority;
						ChangedTypeSet.Add(TypePair.Key.GetName());
					}
					break;
				}
			}

			if (!bFoundAsset)
			{
				FoundPreregister.Emplace(PathToAdd, AssetPriority);
				ChangedTypeSet.Add(TypePair.Key.GetName());
			}
		}
	}

	// If we've initially loaded, apply to all modified types
	if (bFullyInitialized)
	{
		for (const FRegistryMapPair& RegistryPair : RegistryMap)
		{
			if (ChangedTypeSet.Contains(NAME_None) || ChangedTypeSet.Contains(RegistryPair.Key))
			{
				UDataRegistry* Registry = RegistryPair.Value;
				if (Registry)
				{
					ApplyPreregisterMap(Registry);
				}
			}
		}
	}
}

void UDataRegistrySubsystem::ApplyPreregisterMap(UDataRegistry* Registry)
{
	// Apply both the invalid and type-specific lists
	const UDataRegistrySettings* Settings = GetDefault<UDataRegistrySettings>();
	UAssetManager& AssetManager = UAssetManager::Get();
	TArray<FPreregisterAsset>* FoundPreregister = PreregisterAssetMap.Find(Registry->GetRegistryType());

	if (FoundPreregister)
	{
		const FTopLevelAssetPath ObjectClassPath(TEXT("/Script/CoreUObject"), TEXT("Object"));
		for (int32 i = 0; i < FoundPreregister->Num(); i++)
		{
			EDataRegistryRegisterAssetResult RegisterAssetResult = EDataRegistryRegisterAssetResult::NotRegistered;
			bool bRegistered = false;
			const FSoftObjectPath& AssetPath = (*FoundPreregister)[i].Key;
			FAssetData AssetData;
			if (AssetManager.GetAssetDataForPath(AssetPath, AssetData))
			{
				RegisterAssetResult = Registry->RegisterSpecificAsset(AssetData, (*FoundPreregister)[i].Value);
			}
			else if (Settings->CanIgnoreMissingAssetData())
			{
				// Construct fake asset data and register that
				AssetData = FAssetData(AssetPath.GetLongPackageName(), AssetPath.GetAssetPathString(), ObjectClassPath);
				RegisterAssetResult = Registry->RegisterSpecificAsset(AssetData, (*FoundPreregister)[i].Value);
			}

			if (RegisterAssetResult == EDataRegistryRegisterAssetResult::NotRegistered)
			{
				// If specific type is mentioned, it is expected to always succeed unless it was already registered
				UE_LOGF(LogDataRegistry, Warning, "ApplyPreregisterMap failed to register %ls with %ls, there needs to be a meta source that handles registered assets with matching data", *AssetPath.ToString(), *Registry->GetRegistryType().ToString());
			}
			else if (RegisterAssetResult == EDataRegistryRegisterAssetResult::AssetAlreadyRegistered)
			{
				UE_LOGF(LogDataRegistry, Verbose, "ApplyPreregisterMap did not register %ls with %ls, this asset was already registered with the data registry.", *AssetPath.ToString(), *Registry->GetRegistryType().ToString());
			}
		}
	}

	// Now check 
	FoundPreregister = PreregisterAssetMap.Find(FDataRegistryType());

	if (FoundPreregister)
	{
		for (int32 i = 0; i < FoundPreregister->Num(); i++)
		{
			const FSoftObjectPath& AssetPath = (*FoundPreregister)[i].Key;
			FAssetData AssetData;
			if (AssetManager.GetAssetDataForPath(AssetPath, AssetData))
			{
				Registry->RegisterSpecificAsset(AssetData, (*FoundPreregister)[i].Value);
			}
			else if (Settings->CanIgnoreMissingAssetData())
			{
				// This is not safe without a specified type as it can't know which registry is the proper destination
				UE_LOGF(LogDataRegistry, Warning, "ApplyPreregisterMap failed to register %ls, Type must be specified if asset data is missing", *AssetPath.ToString());
			}
		}
	}
}

FDataRegistryCacheGetResult UDataRegistrySubsystem::GetCachedItemRaw(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId) const
{
	const UDataRegistry* FoundRegistry = GetRegistryForType(ItemId.RegistryType);
	if (FoundRegistry)
	{
		return FoundRegistry->GetCachedItemRaw(OutItemMemory, OutItemStruct, ItemId);
	}
	OutItemMemory = nullptr;
	OutItemStruct = nullptr;
	return FDataRegistryCacheGetResult();
}

FDataRegistryCacheGetResult UDataRegistrySubsystem::GetCachedItemRawFromLookup(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup) const
{
	const UDataRegistry* FoundRegistry = GetRegistryForType(ItemId.RegistryType);
	if (FoundRegistry)
	{
		return FoundRegistry->GetCachedItemRawFromLookup(OutItemMemory, OutItemStruct, ItemId, Lookup);
	}
	OutItemMemory = nullptr;
	OutItemStruct = nullptr;
	return FDataRegistryCacheGetResult();
}

bool UDataRegistrySubsystem::AcquireItem(const FDataRegistryId& ItemId, FDataRegistryItemAcquiredCallback DelegateToCall) const
{
	UDataRegistry* FoundRegistry = GetRegistryForType(ItemId.RegistryType);

	if (FoundRegistry)
	{
		return FoundRegistry->AcquireItem(ItemId, DelegateToCall);
	}

	return false;
}

bool UDataRegistrySubsystem::AcquireItemBP(FDataRegistryId ItemId, FDataRegistryItemAcquiredBPCallback AcquireCallback)
{
	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	if (!SubSystem->IsConfigEnabled(true))
	{
		return false;
	}

	// Call the BP delegate, this is safe because it always runs on the game thread on a new frame
	return SubSystem->AcquireItem(ItemId, FDataRegistryItemAcquiredCallback::CreateLambda([AcquireCallback](const FDataRegistryAcquireResult& Result)
		{
			AcquireCallback.ExecuteIfBound(Result.ItemId, Result.ResolvedLookup, Result.Status);
		}));
}

void UDataRegistrySubsystem::GetPossibleDataRegistryIdList(FDataRegistryType RegistryType, TArray<FDataRegistryId>& OutIdList)
{
	OutIdList.Reset();

	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	if (!SubSystem->IsConfigEnabled(true))
	{
		return;
	}

	UDataRegistry* FoundRegistry = SubSystem->GetRegistryForType(RegistryType);

	if (FoundRegistry)
	{
		// Don't sort by default
		FoundRegistry->GetPossibleRegistryIds(OutIdList, false);
	}
}

void UDataRegistrySubsystem::EvaluateDataRegistryCurve(FDataRegistryId ItemId, float InputValue, float DefaultValue, EDataRegistrySubsystemGetItemResult& OutResult, float& OutValue)
{
	UDataRegistrySubsystem* SubSystem = UDataRegistrySubsystem::Get();
	check(SubSystem);

	if (!SubSystem->IsConfigEnabled(true))
	{
		return;
	}

	const FRealCurve* FoundCurve = nullptr;
	if (SubSystem->EvaluateCachedCurve(OutValue, FoundCurve, ItemId, InputValue, DefaultValue))
	{
		OutResult = EDataRegistrySubsystemGetItemResult::Found;
	}
	else
	{
		OutResult = EDataRegistrySubsystemGetItemResult::NotFound;
		OutValue = DefaultValue;
	}
}

FDataRegistryCacheGetResult UDataRegistrySubsystem::EvaluateCachedCurve(float& OutValue, const FRealCurve*& OutCurve, FDataRegistryId ItemId, float InputValue, float DefaultValue) const
{
	UDataRegistry* FoundRegistry = GetRegistryForType(ItemId.RegistryType);
	FDataRegistryCacheGetResult CacheResult;

	if (FoundRegistry)
	{
		CacheResult = FoundRegistry->GetCachedCurveRaw(OutCurve, ItemId);

		if (CacheResult && OutCurve)
		{
			OutValue = OutCurve->Eval(InputValue, DefaultValue);
			return CacheResult;
		}
	}

	// Couldn't find a curve, return default
	OutCurve = nullptr;
	OutValue = DefaultValue;
	return CacheResult;
}

bool UDataRegistrySubsystem::IsValidDataRegistryType(FDataRegistryType DataRegistryType)
{
	return DataRegistryType.IsValid();
}

FString UDataRegistrySubsystem::Conv_DataRegistryTypeToString(FDataRegistryType DataRegistryType)
{
	return DataRegistryType.ToString();
}

bool UDataRegistrySubsystem::EqualEqual_DataRegistryType(FDataRegistryType A, FDataRegistryType B)
{
	return A == B;
}

bool UDataRegistrySubsystem::NotEqual_DataRegistryType(FDataRegistryType A, FDataRegistryType B)
{
	return A != B;
}

bool UDataRegistrySubsystem::IsValidDataRegistryId(FDataRegistryId DataRegistryId)
{
	return DataRegistryId.IsValid();
}

FString UDataRegistrySubsystem::Conv_DataRegistryIdToString(FDataRegistryId DataRegistryId)
{
	return DataRegistryId.ToString();
}

bool UDataRegistrySubsystem::EqualEqual_DataRegistryId(FDataRegistryId A, FDataRegistryId B)
{
	return A == B;
}

bool UDataRegistrySubsystem::NotEqual_DataRegistryId(FDataRegistryId A, FDataRegistryId B)
{
	return A != B;
}

void UDataRegistrySubsystem::PostEngineInit()
{
	UGameplayTagsManager::Get().CallOrRegister_OnDoneAddingNativeTagsDelegate(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UDataRegistrySubsystem::PostGameplayTags));
}

void UDataRegistrySubsystem::PostGameplayTags()
{
	check(UAssetManager::IsInitialized());
	UAssetManager& AssetManager = UAssetManager::Get();
	AssetManager.CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UDataRegistrySubsystem::PostAssetManager));
}

void UDataRegistrySubsystem::PostAssetManager()
{
	// This will happen fast in game builds, can be many seconds in editor builds due to async load of asset registry
	bReadyForInitialization = true;

#if WITH_EDITOR
	if (WantsDelayedDataRegistryLoadingUntilPIE())
	{
		bReadyForInitialization = false;
		bLoadAllRegistriesOnNextPIE = true;
	}
#endif
	if (bReadyForInitialization)
	{
		LoadAllRegistries();
	}
}

void UDataRegistrySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	check(SingletonSubSystem == nullptr);
	SingletonSubSystem = this;

	Super::Initialize(Collection);

	// This should always happen before PostEngineInit
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UDataRegistrySubsystem::PostEngineInit);

#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorDelegates::PreBeginPIE.AddUObject(this, &UDataRegistrySubsystem::PreBeginPIE);
		FEditorDelegates::EndPIE.AddUObject(this, &UDataRegistrySubsystem::EndPIE);
	}
	
#endif
}

void UDataRegistrySubsystem::Deinitialize()
{
	check(SingletonSubSystem == this);

	if (!GExitPurge)
	{
		DeinitializeAllRegistries();
	}

	bReadyForInitialization = false;
	bFullyInitialized = false;

	Super::Deinitialize();

	SingletonSubSystem = nullptr;
}

void UDataRegistrySubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UDataRegistrySubsystem* This = CastChecked<UDataRegistrySubsystem>(InThis);

	if (GIsEditor)
	{
		for (FRegistryMapPair& RegistryPair : This->RegistryMap)
		{
			// In editor builds we mark this as a weak reference so it can be deleted properly
			Collector.MarkWeakObjectReferenceForClearing(reinterpret_cast<UObject**>(&RegistryPair.Value), This);
		}
	}
	else
	{
		for (FRegistryMapPair& RegistryPair : This->RegistryMap)
		{
			Collector.AddReferencedObject(RegistryPair.Value, This);
		}
	}
}

#if WITH_EDITOR

bool UDataRegistrySubsystem::WantsDelayedDataRegistryLoadingUntilPIE() const
{
	return GIsEditor && !IsRunningCommandlet() && GetDefault<UDataRegistrySettings>()->bDelayLoadingDataRegistriesUntilPIE;
}

void UDataRegistrySubsystem::PreBeginPIE(bool bStartSimulate)
{
	FlushDeferredRegistryLoad(true);

	if (IsConfigEnabled())
	{
		RefreshRegistryMap();
		InitializeAllRegistries(true);
	}
}

void UDataRegistrySubsystem::EndPIE(bool bStartSimulate)
{
	if (IsConfigEnabled())
	{
		ResetRuntimeState();
	}
}
#endif

void UDataRegistrySubsystem::FlushDeferredRegistryLoad(bool bExpected) const
{
#if WITH_EDITOR
	if (bLoadAllRegistriesOnNextPIE)
	{
		// Const cast because the flush can be lazy on first access.
		UDataRegistrySubsystem* MutableThis = const_cast<UDataRegistrySubsystem*>(this);
		if (!bExpected)
		{
			UE_LOGF(LogDataRegistry, Warning, "Accessed data registry map before PIE. Loading on demand.");
		}
		
		MutableThis->bReadyForInitialization = true;
		MutableThis->bLoadAllRegistriesOnNextPIE = false;
		MutableThis->LoadAllRegistries();
	}
#endif // WITH_EDITOR
}


static FAutoConsoleCommand CVarDumpRegistryTypeSummary(
	TEXT("DataRegistry.DumpTypeSummary"),
	TEXT("Shows a summary of types known about by the Data Registry system"),
	FConsoleCommandDelegate::CreateStatic(UDataRegistrySubsystem::DumpRegistryTypeSummary),
	ECVF_Cheat);

void UDataRegistrySubsystem::DumpRegistryTypeSummary()
{
	UDataRegistrySubsystem* Subsystem = Get();

	if (Subsystem)
	{
		UE_LOGF(LogDataRegistry, Log, "=========== DataRegistry Type Summary ===========");
		
		TArray<UDataRegistry*> AllRegistries;

		Subsystem->GetAllRegistries(AllRegistries, true);

		for (UDataRegistry* TypeInfo : AllRegistries)
		{
			UE_LOGF(LogDataRegistry, Log, "  %ls Struct: %ls", *TypeInfo->GetRegistryDescription().ToString(), *GetNameSafe(TypeInfo->GetItemStruct()));
		}
	}
}

static FAutoConsoleCommand CVarDumpCachedItems(
	TEXT("DataRegistry.DumpCachedItems"),
	TEXT("Shows a list of every item available cached for the specified registry type. Add All as second parameter to also print value as text"),
	FConsoleCommandWithArgsDelegate::CreateStatic(UDataRegistrySubsystem::DumpCachedItems),
	ECVF_Cheat);

void UDataRegistrySubsystem::DumpCachedItems(const TArray<FString>& Args)
{
	UDataRegistrySubsystem* Subsystem = Get();

	if (Subsystem)
	{
		if (Args.Num() < 1)
		{
			UE_LOGF(LogDataRegistry, Warning, "Too few arguments for DumpCachedItems. Include the registry type");
			return;
		}

		FString RegistryTypeString = Args[0];		
		bool bDumpValues = false;
		if (Args.Num() > 1 && Args[1] == TEXT("All"))
		{
			bDumpValues = true;
		}

		UDataRegistry* FoundRegistry = Subsystem->GetRegistryForType(FName(*RegistryTypeString));
		if (!FoundRegistry)
		{
			UE_LOGF(LogDataRegistry, Warning, "Invalid registry type %ls", *RegistryTypeString);
			return;
		}

		TMap<FDataRegistryId, const uint8*> CachedItems;
		const UScriptStruct* OutStruct = nullptr;
		FDataRegistryCacheGetResult Result = FoundRegistry->GetAllCachedItems(CachedItems, OutStruct);

		if (Result.GetItemStatus() != EDataRegistryCacheGetStatus::NotFound)
		{
			UE_LOGF(LogDataRegistry, Log, "=========== DataRegistry %ls Cached Items ===========", *RegistryTypeString);

			for (const TPair<FDataRegistryId, const uint8*>& Pair : CachedItems)
			{
				FString ValueString;
				if (bDumpValues && OutStruct && Pair.Value)
				{
					OutStruct->ExportText(ValueString, Pair.Value, nullptr, nullptr, PPF_None, nullptr);
				}

				UE_LOGF(LogDataRegistry, Warning, " %ls: %ls", *Pair.Key.ToString(), *ValueString);
			}
		}
		else
		{
			UE_LOGF(LogDataRegistry, Warning, "No cached items found for %ls", *RegistryTypeString);
		}
	}
}


/*
bool ADataRegistryTestActor::TestSyncRead(FDataRegistryId RegistryId)
{

	UDataRegistrySubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();

	if (Subsystem)
	{
		UDataRegistry* Registry = Subsystem->GetRegistryForType(RegistryId.RegistryType);

		if (Registry)
		{
			const uint8* ItemMemory = nullptr;
			const UScriptStruct* ItemStruct = nullptr;

			if (Registry->GetCachedItemRaw(ItemMemory, ItemStruct, RegistryId))
			{
				FString ValueString;
				ItemStruct->ExportText(ValueString, ItemMemory, nullptr, nullptr, 0, nullptr);
				
				UE_LOGF(LogDataRegistry, Display, "DataRegistryTestActor::TestSyncRead succeeded on %ls with %ls", *RegistryId.ToString(), *ValueString);
				
				return true;
			}
			else
			{
				UE_LOGF(LogDataRegistry, Error, "DataRegistryTestActor::TestSyncRead can't find item for %ls!", *RegistryId.ToString());
			}
		}
		else
		{
			UE_LOGF(LogDataRegistry, Error, "DataRegistryTestActor::TestSyncRead can't find registry for type %ls!", *RegistryId.RegistryType.ToString());
		}

	}

	UE_LOGF(LogDataRegistry, Error, "DataRegistryTestActor::TestSyncRead failed!");
	return false;
}

bool ADataRegistryTestActor::TestAsyncRead(FDataRegistryId RegistryId)
{
	UDataRegistrySubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();

	if (Subsystem)
	{
		if (Subsystem->AcquireItem(RegistryId, FDataRegistryItemAcquiredCallback::CreateUObject(this, &ADataRegistryTestActor::AsyncReadComplete)))
		{
			return true;
		}
	}

	UE_LOGF(LogDataRegistry, Error, "DataRegistryTestActor::TestAsyncRead failed!");

	return false;
}

void ADataRegistryTestActor::AsyncReadComplete(const FDataRegistryAcquireResult& Result)
{
	if (Result.Status == EDataRegistryAcquireStatus::AcquireFinished)
	{
		FString ValueString;
		Result.ItemStruct->ExportText(ValueString, Result.ItemMemory, nullptr, nullptr, 0, nullptr);

		UE_LOGF(LogDataRegistry, Display, "DataRegistryTestActor::AsyncReadComplete succeeded on %ls with %ls", *Result.ItemId.ToString(), *ValueString);
	}
	else
	{
		UE_LOGF(LogDataRegistry, Error, "DataRegistryTestActor::AsyncReadComplete can't find item for %ls!", *Result.ItemId.ToString());
	}
}
*/
