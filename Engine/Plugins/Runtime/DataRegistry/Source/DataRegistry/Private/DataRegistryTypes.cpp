// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTypes.h"
#include "DataRegistryTypesPrivate.h"
#include "DataRegistrySource.h"
#include "DataRegistrySettings.h"
#include "DataRegistrySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistryTypes)

#define LOCTEXT_NAMESPACE "DataRegistry"

DEFINE_LOG_CATEGORY(LogDataRegistry);

FString FDataRegistryLookup::ToString() const
{
	TStringBuilder<256> ReturnBuilder;

	uint8 LookupIndex = 255;
	FName ResolvedName;
	for (int32 i = 0; i < GetNum(); i++)
	{
		GetEntry(LookupIndex, ResolvedName, i);

		if (ReturnBuilder.Len() > 0)
		{
			ReturnBuilder << TEXT(";");
		}

		ReturnBuilder << LookupIndex << TEXT(":") << ResolvedName.ToString();
	}

	return ReturnBuilder.ToString();
}

void FDataRegistryAcquireResult::UpdateAcquireStatus(EDataRegistryAcquireStatus& CurrentStatus, EDataRegistryAcquireStatus NewStatus)
{
	// Enum is ordered with increasing state until success, then error codes in degree of severity/certainty
	if (NewStatus > CurrentStatus)
	{
		CurrentStatus = NewStatus;
	}
}

FString FDataRegistrySourceItemId::GetDebugString() const
{
	const UDataRegistrySource* ResolvedSource = CachedSource.Get();
	if (ItemId.IsValid() && ResolvedSource)
	{
		return FString::Printf(TEXT("%s|%s|%s"), *ItemId.ToString(), *ResolvedSource->GetDebugString(), *SourceResolvedName.ToString());
	}
	return FString();
}

// PRIVATE TYPES

FDataRegistryRequestId FDataRegistryRequestId::GetNewRequestId()
{
	static FDataRegistryRequestId CurrentId;

	CurrentId.RequestId++;

	if (CurrentId.RequestId == InvalidId)
	{
		CurrentId.RequestId++;
	}

	return CurrentId;
}

float FCachedDataRegistryItem::GetRelevancy(const FDataRegistryCachePolicy& CachePolicy, float CurrentTime) const
{
	float SecondsSinceAccess = CurrentTime - LastAccessTime;
	bool bHasForceKeep = (CachePolicy.ForceKeepSeconds != 0.0f);
	bool bHasForceRelease = (CachePolicy.ForceReleaseSeconds != 0.0f);

	if (bHasForceKeep && SecondsSinceAccess < CachePolicy.ForceKeepSeconds)
	{
		return 1.0f;
	}
	if (bHasForceRelease && SecondsSinceAccess > CachePolicy.ForceReleaseSeconds)
	{
		return -1.0f;
	}

	if (bHasForceKeep && bHasForceRelease)
	{
		// TODO is it actually helpful to use linear priority, or should it just be a cutoff?
		return FMath::GetRangePct(CachePolicy.ForceReleaseSeconds, CachePolicy.ForceKeepSeconds, SecondsSinceAccess);
	}

	return 0.5f;
}

const UScriptStruct* FCachedDataRegistryItem::GetItemStruct() const
{
	if (ItemSource.IsValid())
	{
		return ItemSource->GetItemStruct();
	}
	return nullptr;
}

FCachedDataRegistryItem::~FCachedDataRegistryItem()
{
	ClearItemMemory();
}

void FCachedDataRegistryItem::ClearItemMemory()
{
	if (ItemMemory)
	{
		const UScriptStruct* ItemStruct = GetItemStruct();
		if (ensure(ItemStruct))
		{
			FreeItemMemory(ItemStruct, ItemMemory);
			ItemMemory = nullptr;
		}
	}
}

uint8* FCachedDataRegistryItem::AllocateItemMemory(const UScriptStruct* ItemStruct)
{
	check(ItemStruct);
	uint8* ItemStructMemory = (uint8*)FMemory::Malloc(ItemStruct->GetStructureSize());
	ItemStruct->InitializeStruct(ItemStructMemory);
	return ItemStructMemory;
}

void FCachedDataRegistryItem::FreeItemMemory(const UScriptStruct* ItemStruct, uint8* ItemMemory)
{
	check(ItemStruct && ItemMemory);
	ItemStruct->DestroyStruct(ItemMemory);
	FMemory::Free(ItemMemory);
}


FCachedDataRegistryItem* FDataRegistryCache::GetCacheEntry(const FDataRegistryLookup& Lookup)
{
	TUniquePtr<FCachedDataRegistryItem>* FoundItem = LookupCache.Find(Lookup);
	if (FoundItem)
	{
		check(*FoundItem);
		return FoundItem->Get();
	}

	return nullptr;
}

FCachedDataRegistryItem& FDataRegistryCache::GetOrCreateCacheEntry(const FDataRegistryLookup& Lookup)
{
	TUniquePtr<FCachedDataRegistryItem>* FoundItem = LookupCache.Find(Lookup);

	if (FoundItem)
	{
		check(*FoundItem);
		return *FoundItem->Get();
	}

	TUniquePtr<FCachedDataRegistryItem> NewItem(new FCachedDataRegistryItem());
	NewItem->LookupKey = Lookup;
	return *LookupCache.Add(Lookup, MoveTemp(NewItem));
}

bool FDataRegistryCache::RemoveCacheEntry(const FDataRegistryLookup& Lookup)
{
	TUniquePtr<FCachedDataRegistryItem>* FoundItem = LookupCache.Find(Lookup);

	if (FoundItem)
	{
		LookupCache.Remove(Lookup);

		// This can invalidate Lookup
		return true;
	}
	return false;
}

FDataRegistryBatchRequest* FDataRegistryCache::GetBatchRequest(const FDataRegistryRequestId& RequestId)
{
	TUniquePtr<FDataRegistryBatchRequest>* FoundItem = BatchRequests.Find(RequestId);
	if (FoundItem)
	{
		check(*FoundItem);
		return FoundItem->Get();
	}

	return nullptr;
}

FDataRegistryBatchRequest& FDataRegistryCache::CreateNewBatchRequest()
{
	TUniquePtr<FDataRegistryBatchRequest> NewItem(new FDataRegistryBatchRequest());
	NewItem->RequestId = FDataRegistryRequestId::GetNewRequestId();
	return *BatchRequests.Add(NewItem->RequestId, MoveTemp(NewItem));
}

bool FDataRegistryCache::RemoveBatchRequest(const FDataRegistryRequestId& RequestId)
{
	TUniquePtr<FDataRegistryBatchRequest>* FoundItem = BatchRequests.Find(RequestId);

	if (FoundItem)
	{
		BatchRequests.Remove(RequestId);

		return true;
	}
	return false;
}

void FDataRegistryCache::ClearCache(bool bClearRequests)
{
	LookupCache.Reset();

	if (bClearRequests)
	{
		BatchRequests.Reset();
	}
}

TArray<FDataRegistryResolverScope::FResolverStackEntry> FDataRegistryResolverScope::ResolverStack;

FDataRegistryResolverScope::FDataRegistryResolverScope(FDataRegistryResolver& ScopeResolver)
	: AddedScopeResolver(&ScopeResolver)
{
	// Only valid to call on game thread
	check(IsInGameThread());
	ResolverStack.Emplace(ScopeResolver);
}

FDataRegistryResolverScope::FDataRegistryResolverScope(const TSharedPtr<FDataRegistryResolver>& ScopeResolver)
	: AddedScopeResolver(ScopeResolver.Get())
{
	// Only valid to call on game thread
	check(IsInGameThread() && ScopeResolver.IsValid());
	ResolverStack.Emplace(ScopeResolver);
}

FDataRegistryResolverScope::~FDataRegistryResolverScope()
{
	if (ensureMsgf(!ResolverStack.IsEmpty() && (AddedScopeResolver == ResolverStack.Last().AsRawPtr), TEXT("FDataRegistryResolverScope has invalid stack depth at destroy! Unsafe to copy scope or add a global resolve while scope is active")))
	{
		ResolverStack.Pop();
	}
	else
	{
		// Don't want to leak a raw pointer, so manually find and remove it here
		for (int32 i = ResolverStack.Num() - 1; i >= 0; --i)
		{
			if (ResolverStack[i].AsRawPtr == AddedScopeResolver)
			{
				ResolverStack.RemoveAt(i);
				break;
			}
		}
	}
}

void FDataRegistryResolverScope::RegisterGlobalResolver(const TSharedPtr<FDataRegistryResolver>& ScopeResolver)
{
	// Only valid to call on game thread
	check(IsInGameThread() && ScopeResolver.IsValid());
	ResolverStack.Add(ScopeResolver);
}

void FDataRegistryResolverScope::UnregisterGlobalResolver(const TSharedPtr<FDataRegistryResolver>& ScopeResolver)
{
	const int32 RemovedIndex = ResolverStack.FindLastByPredicate([ScopeResolver](const FResolverStackEntry& Entry) { return Entry.AsSharedPtr == ScopeResolver; });
	if (ensureMsgf(RemovedIndex != INDEX_NONE, TEXT("FDataRegistryResolverScope::UnregisterGlobalResolver called with resolver that was never registered!")))
	{
		ResolverStack.RemoveAt(RemovedIndex);
	}
}

FDataRegistryResolver* FDataRegistryResolverScope::ResolveNameFromId(FName& OutResolvedName, const FDataRegistryId& ItemId, const class UDataRegistry* Registry, const class UDataRegistrySource* RegistrySource)
{
	for (int32 i = ResolverStack.Num() - 1; i >= 0; i--)
	{
		FResolverStackEntry& Entry = ResolverStack[i];
		if (Entry.AsRawPtr->ResolveIdToName(OutResolvedName, ItemId, Registry, RegistrySource))
		{
			return ResolverStack[i].AsRawPtr;
		}
	}

	return nullptr;
}

bool FDataRegistryResolverScope::IsStackVolatile()
{
	for (int32 i = ResolverStack.Num() - 1; i >= 0; i--)
	{
		if (ResolverStack[i].AsRawPtr->IsVolatile())
		{
			return true;
		}
	}

	return false;
}

bool UDataRegistrySettings::CanIgnoreMissingAssetData() const
{
#if !WITH_EDITORONLY_DATA
	if (bIgnoreMissingCookedAssetRegistryData)
	{
		return true;
	}
#endif
	return false;
}

#if WITH_EDITOR

void UDataRegistrySettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Right now do a full refresh for any change, switch this when there are less critical settings
	if (UDataRegistrySubsystem* Subsystem = UDataRegistrySubsystem::Get())
	{
		Subsystem->ReinitializeFromConfig();
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE


