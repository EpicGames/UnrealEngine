// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryTypes.h"

class UDataRegistrySource;
struct FStreamableHandle;

// Types private to the module, code in DataRegistryTypes.cpp

/** A multi-item cache request, will call callback when everything is done */
struct FDataRegistryBatchRequest
{
	/** Unique request id in cache */
	FDataRegistryRequestId RequestId;

	/** Array of id->resolved items requested */
	TArray<TPair<FDataRegistryId, FDataRegistryLookup> > RequestedItems;

	/** Ones left to process */
	TArray<FDataRegistryLookup> RemainingAcquires;

	/** Callback when all are acquired */
	FDataRegistryBatchAcquireCallback Callback;

	/** Worst result seen so far */
	EDataRegistryAcquireStatus BatchStatus = EDataRegistryAcquireStatus::NotStarted;
};

/** Info for a specific request for a cached item */
struct FCachedDataRegistryRequest
{
	/** Original registry id that was requested */
	FDataRegistryId RequestedId;

	/** Callback to call on completion of request */
	FDataRegistryItemAcquiredCallback Callback;

	/** Batch request that started this */
	FDataRegistryRequestId BatchId;
};

/** Cached information for a single data item */
struct FCachedDataRegistryItem
{
	~FCachedDataRegistryItem();

	/** What this was stored as, here for cross referencing */
	FDataRegistryLookup LookupKey;

	/** Worst availability of all resources */
	EDataRegistryAcquireStatus AcquireStatus = EDataRegistryAcquireStatus::NotStarted;

	/** What source index it is currently checking, if the acquire is a success this will be the source it found it in */
	int32 AcquireLookupIndex = -1;

	/** Allocated instance of item struct */
	uint8* ItemMemory = nullptr;

	/** What source this cached data came from */
	TWeakObjectPtr<UDataRegistrySource> ItemSource;

	/** Handle to assets async loaded as part of loading item */
	TSharedPtr<FStreamableHandle> ResourceHandle;

	/** Worst availability of all resources */
	EDataRegistryAvailability ResourceAvailability = EDataRegistryAvailability::Unknown;

	/** List of active requests */
	TArray<FCachedDataRegistryRequest> ActiveRequests;

	/** Time this was last accessed */
	float LastAccessTime;

	/** Returns a relevancy score based on time/future parameters. <= 0 means always drop, >= 1 means always keep */
	float GetRelevancy(const FDataRegistryCachePolicy& CachePolicy, float CurrentTime) const;

	/** Returns the struct used for item memory */
	const UScriptStruct* GetItemStruct() const;

	/** Resets the item memory and nulls pointer */
	void ClearItemMemory();

	/** Allocate struct that will eventually be stored in this cache */
	static uint8* AllocateItemMemory(const UScriptStruct* ItemStruct);

	/** Allocate struct that will eventually be stored in this cache */
	static void FreeItemMemory(const UScriptStruct* ItemStruct, uint8* ItemMemory);
};

/** Structure for entire cache, here to avoid exposing publicly */
struct FDataRegistryCache
{
	/** Cached data, this is stored for a resolved lookup based on request context */
	TMap<FDataRegistryLookup, TUniquePtr<FCachedDataRegistryItem>> LookupCache;

	/** Active batch requests */
	TSortedMap<FDataRegistryRequestId, TUniquePtr<FDataRegistryBatchRequest>> BatchRequests;

	/** Current Cache version, modified when anything is changed */
	int32 CurrentCacheVersion = 0;

	/** Gets an active cache entry */
	FCachedDataRegistryItem* GetCacheEntry(const FDataRegistryLookup& Lookup);

	/** Get or create a cache entry */
	FCachedDataRegistryItem& GetOrCreateCacheEntry(const FDataRegistryLookup& Lookup);

	/** Deletes a cache entry */
	bool RemoveCacheEntry(const FDataRegistryLookup& Lookup);

	/** Gets an active request */
	FDataRegistryBatchRequest* GetBatchRequest(const FDataRegistryRequestId& RequestId);

	/** Creates a new request with a new unique id */
	FDataRegistryBatchRequest& CreateNewBatchRequest();

	/** Deletes a cache entry */
	bool RemoveBatchRequest(const FDataRegistryRequestId& RequestId);

	/** Clears entire cache */
	void ClearCache(bool bClearRequests);
};
