// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftDataRegistryOrTable.h"

#include "DataRegistrySubsystem.h"
#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoftDataRegistryOrTable)

FSoftDataRegistryOrTable::FSoftDataRegistryOrTable()
	: bUseDataRegistry(false)
	, Table(nullptr)
	, RegistryType(NAME_None)
{
}

FSoftDataRegistryOrTable::FSoftDataRegistryOrTable(const UDataTable* InDataTable, const FDataRegistryType& InRegistryType)
{
	if (InRegistryType != NAME_None)
	{
		RegistryType = InRegistryType;
		Table = nullptr;
		bUseDataRegistry = true;
	}
	else if(InDataTable)
	{ 
		Table = const_cast<UDataTable*>(InDataTable);
		bUseDataRegistry = false;
	}
	else
	{
		RegistryType = NAME_None;
		Table = nullptr;
		bUseDataRegistry = false;
	}
}

bool FSoftDataRegistryOrTable::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() && Ar.IsPersistent())
	{
		if (bUseDataRegistry)
		{
			// clean out table reference if we are using a data registry
			Table = nullptr;
		}
		else
		{
			// clean out the registry type if we aren't using it
			RegistryType = NAME_None;
		}
	}
	
	// return false so the normalize serializer will handle the serialization
	return false;
}

bool FSoftDataRegistryOrTable::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	// Note: This assumes the previous slot was an FDataRegistryType
	if (Tag.GetType().IsStruct(FDataRegistryType::StaticStruct()->GetFName()))
	{
		FDataRegistryType Reference;
		FDataRegistryType::StaticStruct()->SerializeItem(Slot, &Reference, nullptr);
		if (!Reference.GetName().IsNone())
		{
			RegistryType = Reference;
			bUseDataRegistry = true;
			return true;
		}
	}
	
	// NOTE: this code assumes that the previous soft object ptr was for a UDataTable
	if (Tag.Type == NAME_SoftObjectProperty)
	{
		FSoftObjectPtr OldProperty;
		Slot << OldProperty;

		bUseDataRegistry = false;
		Table = OldProperty.ToSoftObjectPath();

		return true;
	}
	else if (Tag.Type == NAME_ObjectProperty)
	{
		TObjectPtr<UDataTable> OldTable;
		Slot << OldTable;

		Table = OldTable;
		bUseDataRegistry = false;

		return true;
	}

	return false;
}

bool FSoftDataRegistryOrTable::IsLoaded() const 
{ 
	return bUseDataRegistry || (!Table.IsNull() && !Table.IsPending()); 
}

void FSoftDataRegistryOrTable::LoadAsync(FStreamableDelegate DelegateToCall)
{
	if (bUseDataRegistry || !Table.IsPending())
	{
		FStreamableHandle::ExecuteDelegate(DelegateToCall);
		return;
	}

	UAssetManager::Get().GetStreamableManager().RequestAsyncLoad(Table.ToSoftObjectPath(), DelegateToCall);
}

FDataRegistryOrTableRow FSoftDataRegistryOrTable::GetRegistryOrTableRow(FName RowName) const
{
	if (bUseDataRegistry)
	{
		// make a data registry id
		return FDataRegistryOrTableRow(FDataRegistryId(RegistryType, RowName));
	}

	// make a data table row handle
	FDataTableRowHandle RowHandle;
	RowHandle.DataTable = Table.Get();
	RowHandle.RowName = RowName;

	return RowHandle;
}

bool FSoftDataRegistryOrTable::Matches(const UDataTable* InTable) const
{
	return (!bUseDataRegistry && (InTable == Table));
}

bool FSoftDataRegistryOrTable::Matches(const UDataRegistry* InRegistry) const
{
	return (bUseDataRegistry && (InRegistry->GetRegistryType() == RegistryType.GetName()));
}

bool FSoftDataRegistryOrTable::Matches(const FDataRegistryOrTableRow& RegistryOrTableId) const
{
	if (RegistryOrTableId.bUseDataRegistryId)
	{
		return Matches(RegistryOrTableId.GetDataRegistry());
	}

	return Matches(RegistryOrTableId.DataTableRow.DataTable);
}

bool FSoftDataRegistryOrTable::IsValid() const
{
	if (bUseDataRegistry)
	{
		return (RegistryType.IsValid());
	}

	return (!Table.IsNull());
}

const UDataRegistry* FSoftDataRegistryOrTable::GetDataRegistry() const
{
	if (!bUseDataRegistry)
	{
		return nullptr;
	}

	UDataRegistrySubsystem* RegistrySystem = UDataRegistrySubsystem::Get();
	if (!ensure(RegistrySystem))
	{
		return nullptr;
	}

	return RegistrySystem->GetRegistryForType(RegistryType);
}

const UScriptStruct* FSoftDataRegistryOrTable::GetStruct() const
{
	if (bUseDataRegistry)
	{
		const UDataRegistry* EventRegistry = GetDataRegistry();
		if (!EventRegistry)
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found  Registry:%ls", __FUNCTION__, *RegistryType.GetName().ToString());
			return nullptr;
		}

		return EventRegistry->GetItemStruct();
	}

	if (UDataTable* TablePtr = Table.LoadSynchronous())
	{
		return TablePtr->GetRowStruct();
	}

	return nullptr;
}

bool FDataRegistryOrTableRow::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FLazyName DataTableRowHandleName("DataTableRowHandle");
	if (Tag.GetType().IsStruct(DataTableRowHandleName))
	{
		// Serialize the DataTableRowHandle
		FDataTableRowHandle OldHandle;
		FDataTableRowHandle::StaticStruct()->SerializeItem(Slot, &OldHandle, nullptr);

		// copy into new struct
		bUseDataRegistryId = false;
		DataTableRow = OldHandle;

		return true;
	}

	return false;
}

const UScriptStruct* FDataRegistryOrTableRow::GetStruct() const
{
	if (bUseDataRegistryId)
	{
		const UDataRegistry* EventRegistry = GetDataRegistry();
		if (!EventRegistry)
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found  Registry:%ls", __FUNCTION__, *DataRegistryId.RegistryType.GetName().ToString());
			return nullptr;
		}

		return EventRegistry->GetItemStruct();
	}

	if (DataTableRow.DataTable)
	{
		return DataTableRow.DataTable->GetRowStruct();
	}

	return nullptr;
}

FDataRegistryOrTableRow::FDataRegistryOrTableRow()
	: bUseDataRegistryId(false)
{
}

FDataRegistryOrTableRow::FDataRegistryOrTableRow(const FDataTableRowHandle& RowHandle)
	: bUseDataRegistryId(false)
	, DataTableRow(RowHandle)
{
}

FDataRegistryOrTableRow::FDataRegistryOrTableRow(const FDataRegistryId& RegistryId)
	: bUseDataRegistryId(true)
	, DataRegistryId(RegistryId)
{
}

FString FDataRegistryOrTableRow::ToString() const
{
	if (bUseDataRegistryId)
	{
		return DataRegistryId.ToString();
	}

	return DataTableRow.ToDebugString();
}

FName FDataRegistryOrTableRow::GetItemName() const
{
	if (bUseDataRegistryId)
	{
		return DataRegistryId.ItemName;
	}

	return DataTableRow.RowName;
}

const UDataRegistry* FDataRegistryOrTableRow::GetDataRegistry() const
{
	if (!bUseDataRegistryId)
	{
		return nullptr;
	}

	UDataRegistrySubsystem* RegistrySystem = UDataRegistrySubsystem::Get();
	if (!ensure(RegistrySystem))
	{
		return nullptr;
	}

	return RegistrySystem->GetRegistryForType(DataRegistryId.RegistryType);
}

bool FDataRegistryOrTableRow::IsValid() const
{
	return bUseDataRegistryId ? DataRegistryId.IsValid() : !DataTableRow.IsNull();
}
