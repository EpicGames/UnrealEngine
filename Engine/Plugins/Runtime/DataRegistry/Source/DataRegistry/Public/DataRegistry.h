// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryTypes.h"
#include "Engine/TimerHandle.h"

#include "DataRegistry.generated.h"

#define UE_API DATAREGISTRY_API

struct FDataRegistryId;
struct FPropertyChangedEvent;

class UDataRegistrySource;
struct FDataRegistryCache;
struct FCachedDataRegistryItem;
struct FRealCurve;

/** 
 * Defines a place to efficiently store and retrieve structure data, can be used as a wrapper around Data/Curve Tables or extended with other sources
 */
UCLASS(MinimalAPI)
class UDataRegistry : public UObject
{
	GENERATED_BODY()
public:

	UE_API UDataRegistry();
	UE_API virtual ~UDataRegistry();

	/** Returns the struct used by this registry, everything returned will be this or a subclass */
	UE_API const UScriptStruct* GetItemStruct() const;

	/** Returns true if this registry struct inherits from a particular named struct */
	UE_API bool DoesItemStructMatchFilter(FName FilterStructName) const;

	/** Gets the formatting for this Id */
	UE_API const FDataRegistryIdFormat& GetIdFormat() const;

	/** Returns the name for type exposed by this registry */
	UE_API const FName GetRegistryType() const;

	/** Gets a human readable summary of registry, for UI usage */
	UE_API virtual FText GetRegistryDescription() const;

	/** Returns true if this state has been initialized for use */
	UE_API virtual bool IsInitialized() const;

	/** Initialize for requests, called when subsystem starts up and should return true on success */
	UE_API virtual bool Initialize();

	/** Disable access and restore to state before initialization, won't do anything if not initialized */
	UE_API virtual void Deinitialize();

	/** Reset caches and state because gameplay finished due to PIE shutting down or the game registering a return to main menu, but stay initialized for future use */
	UE_API virtual void ResetRuntimeState();

	/** Marks this registry for needing a runtime refresh at next opportunity */
	UE_API virtual void MarkRuntimeDirty();

	/** Conditionally refresh the runtime state if needed */
	UE_API virtual void RuntimeRefreshIfNeeded();

	/** 
	 * Attempt to register a specified asset with a source, returns an EDataRegistryRegisterAssetResult, indicating whether the asset was registered or not. Can be used to update priority for existing asset as well 
	 * @return An EDataRegistryRegisterAssetResult that indicates if the asset Was Registered, whether it failed to register, or did not register because the asset was already registered.
	 */
	UE_API virtual EDataRegistryRegisterAssetResult RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority = 0);

	/** Use this to confirm that a DataRegistryRegisterAssetResult caused a change to the Data Registry. Virtual so that different registries can decide what may cause a change.*/
	UE_API virtual bool DidRegisterAssetResultCauseChange(EDataRegistryRegisterAssetResult RegisterAssetStatus) const;

	/** Removes references to a specific asset, returns bool if it was removed */
	UE_API virtual bool UnregisterSpecificAsset(const FSoftObjectPath& AssetPath);

	/** Unregisters all previously registered assets in a specific registry with a specific priority, can be used as a batch reset. Returns number of assets unregistered */
	UE_API virtual int32 UnregisterAssetsWithPriority(int32 AssetPriority);

	/** Returns the worst availability for anything stored in this registry, if this is Precached then this registry is a wrapper around already loaded data */
	UE_API virtual EDataRegistryAvailability GetLowestAvailability() const;

	/** Gets the current general cache policy */
	UE_API const FDataRegistryCachePolicy& GetRuntimeCachePolicy() const;

	/** Sets the current cache policy, could cause items to get released */
	UE_API void SetRuntimeCachePolicy(const FDataRegistryCachePolicy& NewPolicy);

	/** Applies the cache policy, is called regularly but can be manually executed */
	UE_API virtual void ApplyCachePolicy();

	/** Checks if a previous cached get is still valid */
	UE_API virtual bool IsCacheGetResultValid(FDataRegistryCacheGetResult Result) const;

	/** Returns the current cache version for a successful get, may change depending on stack-specific resolve settings */
	UE_API virtual FDataRegistryCacheGetResult GetCacheResultVersion() const;

	/** Bump the cache version from some external event like game-specific file loading */
	UE_API virtual void InvalidateCacheVersion();

	/** Accesses the delegate called when cache version changes */
	UE_API virtual FDataRegistryCacheVersionCallback& OnCacheVersionInvalidated();

	/** Resolves an item id into a specific source and unique id, this can remap the names using game-specific rules. PrecachedDataPtr will be set if it is precached by another system */
	UE_API virtual bool ResolveDataRegistryId(FDataRegistryLookup& OutLookup, const FDataRegistryId& ItemId, const uint8** PrecachedDataPtr = nullptr) const;

	/** Fills in a list of all item ids provided by this registry, sorted for display */
	UE_API virtual void GetPossibleRegistryIds(TArray<FDataRegistryId>& OutRegistryIds, bool bSortForDisplay = true) const;

	/** Start an async request for a single item */
	UE_API virtual bool AcquireItem(const FDataRegistryId& ItemId, FDataRegistryItemAcquiredCallback DelegateToCall);

	/** Start an async request for multiple items*/
	UE_API virtual bool BatchAcquireItems(const TArray<FDataRegistryId>& ItemIds, FDataRegistryBatchAcquireCallback DelegateToCall);

	/** Finds the cached item using a resolved lookup, this can be useful after a load has happened to ensure you get the exact item requested */
	UE_API virtual FDataRegistryCacheGetResult GetCachedItemRawFromLookup(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup) const;

	/** Returns the raw cached data and struct type, useful for generic C++ calls */
	UE_API virtual FDataRegistryCacheGetResult GetCachedItemRaw(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId) const;

	/** Curve wrapper for get function */
	UE_API virtual FDataRegistryCacheGetResult GetCachedCurveRaw(const FRealCurve*& OutCurve, const FDataRegistryId& ItemId) const;

	/** Find the source associated with a lookup index */
	UE_API virtual UDataRegistrySource* LookupSource(FName& OutResolvedName, const FDataRegistryLookup& Lookup, int32 LookupIndex) const;

	/** Finds the cached item, using the request context to handle remapping */
	template <class T>
	const T* GetCachedItem(const FDataRegistryId& ItemId) const
	{
		const uint8* TempItemMemory = nullptr;
		const UScriptStruct* TempItemStuct = nullptr;

		if (GetCachedItemRaw(TempItemMemory, TempItemStuct, ItemId))
		{
			if (!ensureMsgf(TempItemStuct->IsChildOf(T::StaticStruct()), TEXT("Can't cast data item of type %s to %s! Code should check type before calling GetCachedDataRegistryItem"), *TempItemStuct->GetName(), *T::StaticStruct()->GetName()))
			{
				return nullptr;
			}

			return reinterpret_cast<const T*>(TempItemMemory);
		}

		return nullptr;
	}

	/* Method to run the given predicate on all items */
	template <class T>
	void ForEachCachedItem(const FString& ContextString, TFunctionRef<void(const FName& Name, const T& Item)> Predicate) const
	{
		TMap<FDataRegistryId, const uint8*> CachedItemMap;
		const UScriptStruct* ItemTypeStruct = nullptr;

		const FDataRegistryCacheGetResult Result = GetAllCachedItems(CachedItemMap, ItemTypeStruct);
		if (!Result.WasFound())
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry Data found (%ls)  Registry:%ls", __FUNCTION__, *ContextString, *RegistryType.ToString());
			return;
		}

		if (ItemTypeStruct == nullptr || !ItemTypeStruct->IsChildOf(T::StaticStruct()))
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] Registry has incorrect row type (%ls)  Registry:%ls", __FUNCTION__, *ContextString, *RegistryType.ToString());
			return;
		}

		for (TPair<FDataRegistryId, const uint8*>& CachedItem : CachedItemMap)
		{
			const T* Item = reinterpret_cast<const T*>(CachedItem.Value);
			if (Item)
			{
				Predicate(CachedItem.Key.ItemName, *Item);
			}
		}
	}

	/* Method to get all items in the registry */
	template <class T>
	void GetAllItems(const TCHAR* ContextString, TArray<const T*>& Items) const
	{
		TMap<FDataRegistryId, const uint8*> CachedItemMap;
		const UScriptStruct* ItemTypeStruct = nullptr;

		const FDataRegistryCacheGetResult Result = GetAllCachedItems(CachedItemMap, ItemTypeStruct);
		if (!Result.WasFound())
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry Data found (%ls)  Registry:%ls", __FUNCTION__, ContextString, *RegistryType.ToString());
			return;
		}

		if (ItemTypeStruct == nullptr || !ItemTypeStruct->IsChildOf(T::StaticStruct()))
		{
			UE_LOGF(LogDataRegistry, Warning, "[%s] Registry has incorrect row type (%ls)  Registry:%ls", __FUNCTION__, ContextString, *RegistryType.ToString());
			return;
		}

		for (TPair<FDataRegistryId, const uint8*>& CachedItem : CachedItemMap)
		{
			const T* Item = reinterpret_cast<const T*>(CachedItem.Value);
			Items.Add(Item);
		}
	}

	/* Method to get all item names in teh registry */
	UE_API void GetItemNames(TArray<FName>& ItemNames) const;

	/** 
	 * Fills in a map with all cached (and precached) ids and items for fast iteration. 
	 * This will use the current resolve context so may not always be valid, and multiple ids can map to the same raw pointer
	 */
	UE_API virtual FDataRegistryCacheGetResult GetAllCachedItems(TMap<FDataRegistryId, const uint8*>& OutItemMap, const UScriptStruct*& OutItemStruct) const;

	/** Returns the list of data sources. */
	const TArray<TObjectPtr<UDataRegistrySource>>& GetDataSources() const { return DataSources; }

	/** Returns the data sources obtained during runtime. */
	const TArray<TObjectPtr<UDataRegistrySource>>& GetRuntimeSources() const { return RuntimeSources; }

	// Internal functions called from sources and subclasses
	
	/** Gets list of child runtime sources created by passed in source, in order registered */
	UE_API virtual void GetChildRuntimeSources(UDataRegistrySource* ParentSource, TArray<UDataRegistrySource*>& ChildSources) const;

	/** Returns the index of a source in the source list */
	UE_API int32 GetSourceIndex(const UDataRegistrySource* Source, bool bUseRuntimeSources = true) const;

	/** Call to indicate that a item is available, will insert into cache */
	UE_API virtual void HandleAcquireResult(const FDataRegistrySourceAcquireRequest& Request, EDataRegistryAcquireStatus Status, uint8* ItemMemory, UDataRegistrySource* Source);

	/** Returns a timer manager that is safe to use for asset loading actions. This will either be the editor or game instance one, or null during very early startup */
	static UE_API class FTimerManager* GetTimerManager();

	/** Gets the current time */
	static UE_API float GetCurrentTime();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;

	/** Validate and refresh registration */
	UE_API virtual void EditorRefreshRegistry();

	/** Returns all source ids for editor display */
	UE_API virtual void GetAllSourceItems(TArray<FDataRegistrySourceItemId>& OutSourceItems) const;

	/** Request a list of source items */
	UE_API virtual bool BatchAcquireSourceItems(TArray<FDataRegistrySourceItemId>& SourceItems, FDataRegistryBatchAcquireCallback DelegateToCall);
#endif

protected:

	/** Globally unique name used to identify this registry */
	UPROPERTY(EditDefaultsOnly, Category = DataRegistry, AssetRegistrySearchable)
	FName RegistryType;

	/** Rules for specifying valid item Ids, if default than any name can be used */
	UPROPERTY(EditDefaultsOnly, Category = DataRegistry)
	FDataRegistryIdFormat IdFormat;

	/** Structure type of all for items in this registry */
	UPROPERTY(EditDefaultsOnly, Category = DataRegistry, AssetRegistrySearchable, meta = (DisplayThumbnail = "false"))
	TObjectPtr<const UScriptStruct> ItemStruct;

	/** List of data sources to search for items */
	UPROPERTY(EditDefaultsOnly, Instanced, Category = DataRegistry)
	TArray<TObjectPtr<UDataRegistrySource>> DataSources;

	// TODO remove VisibleDefaultsOnly or figure out how to stop it from letting you edit the instance properties
	/** Runtime list of data sources, created from above list and includes sources added at runtime */
	UPROPERTY(VisibleDefaultsOnly, Instanced, Transient, Category = DataRegistry)
	TArray<TObjectPtr<UDataRegistrySource>> RuntimeSources;

	/** How often to check for cache updates */
	UPROPERTY(EditDefaultsOnly, Category = Cache)
	float TimerUpdateFrequency = 1.0f;

	/** Editor-set cache policy */
	UPROPERTY(EditDefaultsOnly, Category = Cache)
	FDataRegistryCachePolicy DefaultCachePolicy;

	/** Runtime override */
	FDataRegistryCachePolicy RuntimeCachePolicy;

	/** Callback for when cache version changes, might be moved later */
	FDataRegistryCacheVersionCallback OnCacheVersionInvalidatedCallback;

	/** Refresh the RuntimeSources list */
	UE_API virtual void RefreshRuntimeSources();

	/** Called on timer tick when initialized */
	UE_API virtual void TimerUpdate();

	/** Maps from a type:name pair to a per-source resolved name, default just returns the name */
	UE_API virtual FName MapIdToResolvedName(const FDataRegistryId& ItemId, const UDataRegistrySource* RegistrySource) const;

	/** Adds all possible source ids for resolved name to set, regardless of request context, default just uses the name */
	UE_API virtual void AddAllIdsForResolvedName(TSet<FDataRegistryId>& PossibleIds, const FName& ResolvedName, const UDataRegistrySource* RegistrySource) const;

	/** Returns internal cached data or raw memory ptr for precached data */
	UE_API const FCachedDataRegistryItem* FindCachedData(const FDataRegistryId& ItemId, const uint8** PrecachedDataPtr = nullptr) const;

	/** Advances state machine for a cached entry */
	UE_API virtual void HandleCacheEntryState(const FDataRegistryLookup& Lookup, FCachedDataRegistryItem& CachedEntry);

	/** Handle sending completion/error callbacks */
	UE_API virtual void HandleAcquireCallbacks(const FDataRegistryLookup& Lookup, FCachedDataRegistryItem& CachedEntry);

	/** Check on any batch requests that need processing, if filter is empty will process all */
	UE_API virtual void UpdateBatchRequest(const FDataRegistryRequestId& RequestId, const FDataRegistryAcquireResult& Result);

	/** Frame-delayed callback to call success for already loaded items */
	UE_API virtual void DelayedHandleSuccessCallbacks(FDataRegistryLookup Lookup);

	/** Start an async request */
	UE_API virtual bool AcquireItemInternal(const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup, FDataRegistryItemAcquiredCallback DelegateToCall, const FDataRegistryRequestId& BatchRequestId);


	// Overrides
	UE_API virtual void BeginDestroy() override;
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:
	/** Internal cache data, can't use TUniquePtr due to UObject weirdness */
	FDataRegistryCache* Cache = nullptr;

	FTimerHandle UpdateTimer;

	/** True if this registry has been initialized and is expected to respond to requests */
	bool bIsInitialized = false;

	/** True if this registry needs a runtime refresh due to asset changes */
	bool bNeedsRuntimeRefresh = false;

};

#undef UE_API
