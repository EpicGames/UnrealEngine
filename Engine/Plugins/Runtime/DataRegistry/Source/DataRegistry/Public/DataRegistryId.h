// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryId.generated.h"

/**
 * A DataRegistry item is an arbitrary USTRUCT representing a specific piece of data, that can be acquired from many possible sources, cached, and returned to requesting code.
 * It is similar in concept to a DataTable Row, and will often literally be one.
 *
 * Here is the asynchronous data flow to go from DataRegistryId to a DataRegistryItem:
 * 1. DataRegistryId is constructed with either a special editor customization or at runtime. Can be a gameplay tag or raw name
 * 2. User tells DataRegistrySubsystem to start an async request to fetch a DataRegistryItem using the Id, and passes in a context object
 * 3. Subsystem finds the right DataRegistry by looking at the type and forwards the request
 * 4. Registry resolves the id into DataRegistryLookup using the context object and game specific logic (ex: converts tag Banner.Foo.Bar into Foo_Bar in a local datatable)
 * 5. Registry looks in its cache for that lookup, if it is found and valid then it schedules success callback for next frame
 * 6. If not found in cache, use lookup to search DataRegistrySources in order
 * 7. Source looks for item, if finding it requires loading an asset or accessing the internet this may take a long time
 * 8. If source fails to find item, try next source until exhausted. If all exhausted, schedule failure callback
 * 9. If source finds item, cache found result into Registry, then schedule success callback
 * 10. Once asset is in cache due to previous async call or always loaded assets, it can be accessed directly via the subsystem cache get functions
 */


/**
 * Wrapper struct to represent a global data registry, represented as an FName internally and implicitly convertible back and forth.
 * This exists so the blueprint API can understand it's not a normal FName.
 */
USTRUCT(BlueprintType)
struct FDataRegistryType
{
	GENERATED_BODY()

	/** MetaData tag for both Type and Id that is used to restrict available registries to a certain item struct base class, ex. meta=(ItemStruct="NativeBaseClass") */
	DATAREGISTRY_API static const FName ItemStructMetaData;

	/** Fake type that can be used in the picker UI to specify that the Id contains a context-specific name that will be interpreted later */
	DATAREGISTRY_API static const FDataRegistryType CustomContextType;


	/** Constructors, one for each FName constructor */
	inline FDataRegistryType() {}
	inline FDataRegistryType(FName InName) : Name(InName) {}
	inline FDataRegistryType(EName InName) : Name(FName(InName)) {}
	inline FDataRegistryType(const WIDECHAR* InName) : Name(FName(InName)) {}
	inline FDataRegistryType(const ANSICHAR* InName) : Name(FName(InName)) {}

	/** Implicitly convert to FName */
	inline operator FName& () { return Name; }
	inline operator const FName& () const { return Name; }

	/** Returns internal Name explicitly, not normally needed */
	inline FName GetName() const
	{
		return Name;
	}

	inline bool operator==(const FDataRegistryType& Other) const
	{
		return Name == Other.Name;
	}

	inline bool operator!=(const FDataRegistryType& Other) const
	{
		return Name != Other.Name;
	}

	/** Returns true if this is a valid Type */
	inline bool IsValid() const
	{
		return Name != NAME_None;
	}

	/** Returns string version of this Type */
	inline FString ToString() const
	{
		return Name.ToString();
	}

	/** UStruct Overrides */
	DATAREGISTRY_API bool ExportTextItem(FString& ValueStr, FDataRegistryType const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	DATAREGISTRY_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	DATAREGISTRY_API bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

	friend inline uint32 GetTypeHash(const FDataRegistryType& Key)
	{
		return GetTypeHash(Key.Name);
	}

protected:

	/** The FName representing this type */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = DataRegistryType)
	FName Name;
};

template<>
struct TStructOpsTypeTraits<FDataRegistryType> : public TStructOpsTypeTraitsBase2<FDataRegistryType>
{
	enum
	{
		WithZeroConstructor = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/** Identifier for a specific DataRegistryItem, provides the user with a Tag or dropdown-based UI for selecting based on the available index info */
USTRUCT(BlueprintType)
struct FDataRegistryId
{
	GENERATED_BODY()

	/** The type of this item, used to look up the correct registry */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = DataRegistry)
	FDataRegistryType RegistryType;

	/** The name of this object, may be a leaf gameplay tag or a raw name depending on the type */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = DataRegistry)
	FName ItemName;


	/** Explicit constructors */
	inline FDataRegistryId() {}

	/** Construct from two names, will also work with FDataRegistryType due to the implicit cast */
	inline FDataRegistryId(const FName& InRegistryType, const FName& InItemName)
		: RegistryType(InRegistryType)
		, ItemName(InItemName)
	{}
	
	/** Parses out the Type:Name format */
	inline explicit FDataRegistryId(const FString& TypeAndName)
		: FDataRegistryId(ParseTypeAndName(TypeAndName))
	{}

	/** True if this represents an actual registry item */
	inline bool IsValid() const
	{
		return RegistryType != NAME_None && ItemName != NAME_None;
	}

	/** Returns string version of this identifier in Type:Name format */
	DATAREGISTRY_API FString ToString() const;

	/** Returns Text description of string */
	DATAREGISTRY_API FText ToText() const;

	inline bool operator==(const FDataRegistryId& Other) const
	{
		return RegistryType == Other.RegistryType && ItemName == Other.ItemName;
	}

	inline bool operator!=(const FDataRegistryId& Other) const
	{
		return RegistryType != Other.RegistryType || ItemName != Other.ItemName;
	}


	/** Convert from Type:Name strings, these are inline to avoid linker errors for constructor */
	inline static FDataRegistryId ParseTypeAndName(const TCHAR* TypeAndName, uint32 Len)
	{
		for (uint32 Idx = 0; Idx < Len; ++Idx)
		{
			if (TypeAndName[Idx] == ':')
			{
				FName Type(Idx, TypeAndName);
				FName Name(Len - Idx - 1, TypeAndName + Idx + 1);
				return FDataRegistryId(Type, Name);
			}
		}

		return FDataRegistryId();
	}

	inline static FDataRegistryId ParseTypeAndName(FName TypeAndName)
	{
		TCHAR Str[FName::StringBufferSize];
		uint32 Len = TypeAndName.ToString(Str);
		return ParseTypeAndName(Str, Len);
	}

	inline static FDataRegistryId ParseTypeAndName(const FString& TypeAndName)
	{
		return ParseTypeAndName(*TypeAndName, static_cast<uint32>(TypeAndName.Len()));
	}


	/** UStruct Overrides */
	DATAREGISTRY_API bool ExportTextItem(FString& ValueStr, FDataRegistryId const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	DATAREGISTRY_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	DATAREGISTRY_API bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	DATAREGISTRY_API void PostSerialize(const FArchive& Ar); 

	friend inline uint32 GetTypeHash(const FDataRegistryId& Key)
	{
		uint32 Hash = 0;

		Hash = HashCombine(Hash, GetTypeHash(Key.RegistryType));
		Hash = HashCombine(Hash, GetTypeHash(Key.ItemName));
		return Hash;
	}
};

template<>
struct TStructOpsTypeTraits<FDataRegistryId> : public TStructOpsTypeTraitsBase2<FDataRegistryId>
{
	enum
	{
		WithZeroConstructor = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
		WithPostSerialize = true,
	};
};
