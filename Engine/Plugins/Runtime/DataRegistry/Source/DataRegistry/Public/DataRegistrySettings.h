// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "DataRegistrySettings.generated.h"

#define UE_API DATAREGISTRY_API

struct FDirectoryPath;

struct FPropertyChangedEvent;


/** Settings for the Data Registry subsystem, these settings are used to scan for registry assets and set runtime access rules */
UCLASS(MinimalAPI, config = Game, defaultconfig, meta = (DisplayName = "Data Registry"))
class UDataRegistrySettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	
	/** List of directories to scan for data registry assets */
	UPROPERTY(config, EditAnywhere, Category = "Data Registry", meta = (RelativeToGameContentDir, LongPackageName))
	TArray<FDirectoryPath> DirectoriesToScan;

	/** If false, only registry assets inside DirectoriesToScan will be initialized. If true, it will also initialize any in-memory DataRegistry assets outside the scan paths */
	UPROPERTY(config, EditAnywhere, Category = "Data Registry")
	bool bInitializeAllLoadedRegistries = false;

	/** If true, cooked builds will ignore errors with missing AssetRegistry data for specific registered assets like DataTables as it may have been stripped out */
	UPROPERTY(config, EditAnywhere, Category = "Data Registry")
	bool bIgnoreMissingCookedAssetRegistryData = false;

	/** If true, in the editor data registry assets will be loaded before the first PIE instead of during startup */
	UPROPERTY(config, EditAnywhere, Category = "Data Registry")
	bool bDelayLoadingDataRegistriesUntilPIE = false;


	/** Return true if we are allowed to ignore missing asset registry data based on settings and build */
	UE_API bool CanIgnoreMissingAssetData() const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};

#undef UE_API
