// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistry.h"

#include "Engine/AssetManagerTypes.h"
#include "UObject/Package.h"
#include "DataRegistrySource.generated.h"

#define UE_API DATAREGISTRY_API

enum class EDataRegistryAcquireStatus : uint8;
enum class EDataRegistryAvailability : uint8;
struct FDataRegistrySourceAcquireRequest;
template <typename T> class TSubclassOf;

/** Specifies a source for DataRegistry items, which is an interface that provides asynchronous access to individual structs */
UCLASS(MinimalAPI, Abstract, EditInlineNew, DefaultToInstanced, Within = DataRegistry, CollapseCategories)
class UDataRegistrySource : public UObject
{
	GENERATED_BODY()
public:

	/** The outer for this should always be a Registry */
	UE_API const UDataRegistry* GetRegistry() const;
	UE_API UDataRegistry* GetRegistry();

	/** Gets structure from parent registry */
	UE_API const UScriptStruct* GetItemStruct() const;

	/** Returns how available this data is generally */
	UE_API virtual EDataRegistryAvailability GetSourceAvailability() const;

	/** Returns availability of individual item, also get raw memory address if available */
	UE_API virtual EDataRegistryAvailability GetItemAvailability(const FName& ResolvedName, const uint8** PrecachedDataPtr) const;

	/** Fills in set of all names provided by this source */
	UE_API virtual void GetResolvedNames(TArray<FName>& Names) const;

	/** Returns true if this state has been initialized for use */
	UE_API virtual bool IsInitialized() const;

	/** Called on editor-defined sources to initialize this source so it is ready to take requests */
	UE_API virtual bool Initialize();

	/** Called on editor-defined sources to disable access and restore to state before initialization, won't do anything if not initialized */
	UE_API virtual void Deinitialize();

	/** Called to regenerate runtime sources if needed, won't do anything for simple sources */
	UE_API virtual void RefreshRuntimeSources();

	/** Gets the list of runtime sources that should be registered for this source, will either be itself or a list of children */
	UE_API virtual void AddRuntimeSources(TArray<UDataRegistrySource*>& OutRuntimeSources);

	/** Called on runtime sources to reset caches and related state because game has ended or registry settings have changed */
	UE_API virtual void ResetRuntimeState();

	/** Called by owning source as periodic update, can release resources or refresh connections */
	UE_API virtual void TimerUpdate(float CurrentTime, float TimerUpdateFrequency);

	/** Call to indicate that a item is available, will insert into cache */
	UE_API virtual void HandleAcquireResult(const FDataRegistrySourceAcquireRequest& Request, EDataRegistryAcquireStatus Status, uint8* ItemMemory);

	/** Call to start an acquire request */
	UE_API virtual bool AcquireItem(FDataRegistrySourceAcquireRequest&& Request);

	/** Return a useful debug name for this source */
	UE_API virtual FString GetDebugString() const;

	/** Returns the path to the underlying source asset (data table, curve table, etc.) if there is one */
	UE_API virtual FSoftObjectPath GetSourceAssetPath() const;

	/** Returns true if this is a runtime-only source */
	UE_API virtual bool IsTransientSource() const;

	/** Resets transient source so it can be correctly garbage collected */
	UE_API virtual bool ResetTransientSource();

	/** Returns the editor-defined source, which is either this or the parent source */
	UE_API virtual UDataRegistrySource* GetOriginalSource();

	/** Returns true if this asset is already registered with this source */
	UE_API virtual bool IsSpecificAssetRegistered(const FSoftObjectPath& AssetPath) const;

	/**
	 * Attempt to register a specified asset with a source, returns an EDataRegistryRegisterAssetResult, indicating whether the asset was registered or not. Can be used to update priority for existing asset as well
	 * @return An EDataRegistryRegisterAssetResult that indicates if the asset Was Registered, whether it failed to register, or did not register because the asset was already registered.
	 */
	UE_API virtual EDataRegistryRegisterAssetResult RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority = 0);

	/** Removes references to a specific asset, returns bool if it was removed */
	UE_API virtual bool UnregisterSpecificAsset(const FSoftObjectPath& AssetPath);

	/** Unregisters all previously registered assets in a specific registry with a specific priority, can be used as a batch reset. Returns number of assets unregistered */
	UE_API virtual int32 UnregisterAssetsWithPriority(int32 AssetPriority);

#if WITH_EDITOR
	/** Called on editor-defined states to check to check a source is still valid after major changes */
	UE_API virtual void EditorRefreshSource();

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	/** Creates a new runtime source, but does not register it yet */
	UE_API UDataRegistrySource* CreateTransientSource(TSubclassOf<UDataRegistrySource> SourceClass);

	/** True if this is currently considered to be initialized */
	bool bIsInitialized = false;

	/** What data source we were created from, if this is a transient source */
	UPROPERTY()
	TObjectPtr<UDataRegistrySource> ParentSource = nullptr;
};


/** Rules specifying how a meta source will deal with assets, arranged as a semi-bitfield */
UENUM()
enum class EMetaDataRegistrySourceAssetUsage : uint8
{
	/** Does not use assets, gets sources from somewhere else like a web database */
	NoAssets = 0,

	/** Only loads assets off disk, does not accept registered assets */
	SearchAssets = 1,

	/** Only accepts registered assets, does not do any scanning */
	RegisterAssets = 2,

	/** Both does search and will accept registered assets, using search rules as filter */
	SearchAndRegisterAssets = 3,
};
ENUM_CLASS_FLAGS(EMetaDataRegistrySourceAssetUsage);

/** Base class for a data source that generates additional data sources at runtime */
UCLASS(MinimalAPI, Abstract)
class UMetaDataRegistrySource : public UDataRegistrySource
{
	GENERATED_BODY()

public:
	/** Returns what subclass of source to use for children, must be overridden! */
	UE_API virtual TSubclassOf<UDataRegistrySource> GetChildSourceClass() const;

	/** Fills in new or existing child source for specific name, must be overridden! */
	UE_API virtual bool SetDataForChild(FName SourceName, UDataRegistrySource* ChildSource);

	/** Fills in list of desired runtime names, must be overridden! */
	UE_API virtual void DetermineRuntimeNames(TArray<FName>& OutRuntimeNames);

	/** Returns true if this asset data passes the filter for this meta source, bNewRegisteredAsset is true if it comes from a RegisterSpecificAsset call and needs extra path checking */
	UE_API virtual bool DoesAssetPassFilter(const FAssetData& AssetData, bool bNewRegisteredAsset);

	// Source Interface
	UE_API virtual void RefreshRuntimeSources() override;
	UE_API virtual void AddRuntimeSources(TArray<UDataRegistrySource*>& OutRuntimeSources) override;
	UE_API virtual bool IsSpecificAssetRegistered(const FSoftObjectPath& AssetPath) const override;
	UE_API virtual EDataRegistryRegisterAssetResult RegisterSpecificAsset(const FAssetData& AssetData, int32 AssetPriority) override;
	UE_API virtual bool UnregisterSpecificAsset(const FSoftObjectPath& AssetPath) override;
	UE_API virtual int32 UnregisterAssetsWithPriority(int32 AssetPriority) override;

	/** Asset usage */
	UPROPERTY(EditAnywhere, Category = DataRegistry)
	EMetaDataRegistrySourceAssetUsage AssetUsage;

	/** Asset registry scan rules */
	UPROPERTY(EditAnywhere, Category = DataRegistry, Meta = (EditCondition = "AssetUsage != EMetaDataRegistrySourceAssetUsage::NoAssets"))
	FAssetManagerSearchRules SearchRules;

protected:

	/** Callback registered when an asset search root as been added post launch */
	UE_API virtual void OnNewAssetSearchRoot(const FString& SearchRoot);

	/** Sort assets, default order is by priority then array order */
	UE_API virtual void SortRegisteredAssets();

	/** Map from source identifier such as package name to registered child */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UDataRegistrySource>> RuntimeChildren;

	/** List of desired source ids, in order */
	TArray<FName> RuntimeNames;

	/** List of specific assets registered with source, in runtime order sorted by priority */
	typedef TPair<FAssetData, int32> FRegisteredAsset;
	TArray<FRegisteredAsset> SpecificRegisteredAssets;

	/** Delegate handle for OnNewAssetSearchRoot */
	FDelegateHandle NewAssetSearchRootHandle;
};

#undef UE_API
