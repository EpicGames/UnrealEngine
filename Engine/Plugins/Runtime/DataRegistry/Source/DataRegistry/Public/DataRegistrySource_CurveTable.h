// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistrySource_DataTable.h"
#include "DataRegistrySource_CurveTable.generated.h"

#define UE_API DATAREGISTRY_API

class UCurveTable;


/** Data source that loads from a specific curve table asset */
UCLASS(MinimalAPI, Meta = (DisplayName = "CurveTable Source"))
class UDataRegistrySource_CurveTable : public UDataRegistrySource
{
	GENERATED_BODY()
public:

	/** What table to load from */
	UPROPERTY(EditAnywhere, Category = DataRegistry)
	TSoftObjectPtr<UCurveTable> SourceTable;

	/** Access rules */
	UPROPERTY(EditAnywhere, Category = DataRegistry)
	FDataRegistrySource_DataTableRules TableRules;

	/** Update source table and rules, meant to be called from a meta source */
	UE_API void SetSourceTable(const TSoftObjectPtr<UCurveTable>& InSourceTable, const FDataRegistrySource_DataTableRules& InTableRules);

protected:
	/** Hard ref to loaded table */
	UPROPERTY(Transient)
	TObjectPtr<UCurveTable> CachedTable;

	/** Preload table ref, will be set if this is a hard source */
	UPROPERTY()
	TObjectPtr<UCurveTable> PreloadTable;

	/** Last time this was accessed */
	mutable float LastAccessTime;

	/** This is set if the table fails to load or could never load */
	bool bInvalidSourceTable = false;

	/** Handle for in progress load */
	TSharedPtr<FStreamableHandle> LoadingTableHandle;

	/** List of requests to resolve when table is loaded */
	TArray<FDataRegistrySourceAcquireRequest> PendingAcquires;

	/** Tells it to set CachedTable if possible */
	UE_API virtual void SetCachedTable(bool bForceLoad = false);

	/** Clears cached table pointer so it can be GCd */
	UE_API virtual void ClearCachedTable();

	/** Tells it to go through each pending acquire */
	UE_API virtual void HandlePendingAcquires();

	/** Callback after table loads */
	UE_API virtual void OnTableLoaded();


	// Source interface
	UE_API virtual EDataRegistryAvailability GetSourceAvailability() const override;
	UE_API virtual EDataRegistryAvailability GetItemAvailability(const FName& ResolvedName, const uint8** PrecachedDataPtr) const override;
	UE_API virtual void GetResolvedNames(TArray<FName>& Names) const override;
	UE_API virtual void ResetRuntimeState() override;
	UE_API virtual bool AcquireItem(FDataRegistrySourceAcquireRequest&& Request) override;
	UE_API virtual void TimerUpdate(float CurrentTime, float TimerUpdateFrequency) override;
	UE_API virtual FString GetDebugString() const override;
	UE_API virtual FSoftObjectPath GetSourceAssetPath() const override;
	UE_API virtual bool Initialize() override;

	// Object interface
	UE_API virtual void PostLoad() override;

	UE_API virtual void OnDataTableChanged();

#if WITH_EDITOR
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void EditorRefreshSource() override;
#endif
};


/** Meta source that will generate CurveTable sources at runtime based on a directory scan or asset registration */
UCLASS(MinimalAPI, Meta = (DisplayName = "CurveTable Meta Source"))
class UMetaDataRegistrySource_CurveTable : public UMetaDataRegistrySource
{
	GENERATED_BODY()
public:
	/** Constructor */
	UE_API UMetaDataRegistrySource_CurveTable();

	/** What specific source class to spawn */
	UPROPERTY(EditAnywhere, Category = DataRegistry)
	TSubclassOf<UDataRegistrySource_CurveTable> CreatedSource;

	/** Access rules */
	UPROPERTY(EditAnywhere, Category = DataRegistry)
	FDataRegistrySource_DataTableRules TableRules;

protected:
	
	// Source interface
	UE_API virtual TSubclassOf<UDataRegistrySource> GetChildSourceClass() const override;
	UE_API virtual bool SetDataForChild(FName SourceId, UDataRegistrySource* ChildSource) override;
	UE_API virtual bool DoesAssetPassFilter(const FAssetData& AssetData, bool bNewRegisteredAsset) override;

};

#undef UE_API
