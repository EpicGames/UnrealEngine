// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistry.h" // IWYU pragma: keep
#include "Subsystems/EngineSubsystem.h"
#include "DataRegistrySubsystem.generated.h"

#define UE_API DATAREGISTRY_API

class UDataRegistry;
struct FRealCurve;
struct FTableRowBase;


/** Enum used to indicate success or failure of finding a data registry item */
UENUM()
enum class EDataRegistrySubsystemGetItemResult : uint8
{
	/** Found the row successfully. */
	Found,
	/** Failed to find the row. */
	NotFound,
};

/** Singleton manager that provides synchronous and asynchronous access to data registries */
UCLASS(MinimalAPI, NotBlueprintType)
class UDataRegistrySubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:

	// Blueprint Interface, it is static for ease of use in custom nodes

	/**
	 * Attempts to get cached structure data stored in a DataRegistry, modifying OutItem if the item is available.
	 * This version has an input param and simple bool return.
	 *
	 * @param ItemID		Item identifier to lookup in cache
	 * @param OutItem		This must be the same type as the registry, if the item is found this will be filled in with the found data
	 * @returns				Returns true if the item was found and OutItem was modified
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = DataRegistry, meta = (DisplayName = "Get Data Registry Item", CustomStructureParam = "OutItem"))
	static bool GetCachedItemBP(FDataRegistryId ItemId, UPARAM(ref) FTableRowBase& OutItem) { return false; }
	DECLARE_FUNCTION(execGetCachedItemBP);

	/**
	 * Attempts to get cached structure data stored in a DataRegistry, returning OutItem if the item is available.
	 * This version has two output pins for convenience, and OutItem should not be accessed from the Not Found pin.
	 *
	 * @param ItemID		Item identifier to lookup in cache
	 * @param OutItem		This must be the same type as the registry, if the item is found this will be filled in with the found data
	 * @param OutResult		Pick execution pin based on if the item was found and OutItem is valid
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = DataRegistry, meta = (DisplayName = "Find Data Registry Item", CustomStructureParam = "OutItem", ExpandEnumAsExecs = "OutResult"))
	static void FindCachedItemBP(FDataRegistryId ItemId, EDataRegistrySubsystemGetItemResult& OutResult, FTableRowBase& OutItem) {}
	DECLARE_FUNCTION(execFindCachedItemBP);

	/**
	 * Attempts to get structure data stored in a DataRegistry cache after an async acquire, returning OutItem if the item is available.
	 * OutItem should not be accessed from the Not Found pin was not found.
	 *
	 * @param ItemID			Item identifier to lookup in cache
	 * @param ResolvedLookup	Resolved identifier returned by acquire function
	 * @param OutItem			This must be the same type as the registry, if the item is found this will be filled in with the found data
	 * @param OutResult			Pick execution pin based on if the item was found and OutItem is valid
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = DataRegistry, meta = (DisplayName = "Find Data Registry Item From Lookup", CustomStructureParam = "OutItem", ExpandEnumAsExecs = "OutResult"))
	static void FindCachedItemFromLookupBP(FDataRegistryId ItemId, const FDataRegistryLookup& ResolvedLookup, EDataRegistrySubsystemGetItemResult& OutResult, FTableRowBase& OutItem) {}
	DECLARE_FUNCTION(execFindCachedItemFromLookupBP);

	/** Deprecated in favor of FindCachedItemFromLookupBP, but does still work properly */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = DataRegistry, meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use Find Data Registry Item From Lookup instead", DisplayName = "Get Data Registry Item From Lookup", CustomStructureParam = "OutItem"))
	static bool GetCachedItemFromLookupBP(FDataRegistryId ItemId, const FDataRegistryLookup& ResolvedLookup, FTableRowBase& OutItem) { return false; }
	DECLARE_FUNCTION(execGetCachedItemFromLookupBP);

	/**
	 * Starts an asynchronous acquire of a data registry item that may not yet be cached, and then accessed with Get Data Registry Item From Lookup
	 * This function will only work properly if the data registry is set up for asynchronous querying.
	 *
	 * @param ItemID			Item identifier to lookup in cache
	 * @param AcquireCallback	Delegate that will be called after acquire succeeds or failed
	 * @returns					Returns true if request was started, false on unrecoverable error
	 */
	UFUNCTION(BlueprintCallable, Category = DataRegistry, meta = (DisplayName = "Acquire Data Registry Item") )
	static UE_API bool AcquireItemBP(FDataRegistryId ItemId, FDataRegistryItemAcquiredBPCallback AcquireCallback);

	/** 
	 * Returns the list of known identifiers for an active data registry so they can be iterated with Find or Acquire.
	 * Depending on how the registry is setup, this could be a large number of identifiers and they may not all be available.
	 * 
	 * @param RegistryType	The type of data registry to query
	 * @param OutIdList		The list of known identifiers for the type, which will be empty if the type is not registered
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = DataRegistry, meta = (DisplayName = "Get Possible Data Registry Id List"))
	static UE_API void GetPossibleDataRegistryIdList(FDataRegistryType RegistryType, TArray<FDataRegistryId>& OutIdList);

	/**
	 * Attempts to evaluate a curve stored in a DataRegistry cache using a specific input value
	 *
	 * @param ItemID		Item identifier to lookup in cache
	 * @param InputValue	Time/level/parameter input value used to evaluate curve at certain position
	 * @param DefaultValue	Value to use if no curve found or input is outside acceptable range
	 * @param OutValue		Result will be replaced with evaluated value, or default if that fails
	 */
	UFUNCTION(BlueprintCallable, Category = DataRegistry, meta = (ExpandEnumAsExecs = "OutResult"))
	static UE_API void EvaluateDataRegistryCurve(FDataRegistryId ItemId, float InputValue, float DefaultValue, EDataRegistrySubsystemGetItemResult& OutResult, float& OutValue);


	/** Returns true if this is a non-empty type, does not check if it is currently registered */
	UFUNCTION(BlueprintPure, Category = DataRegistry, meta = (ScriptMethod = "IsValid", ScriptOperator = "bool", BlueprintThreadSafe))
	static UE_API bool IsValidDataRegistryType(FDataRegistryType DataRegistryType);

	/** Converts a Data Registry Type to a string. The other direction is not provided because it cannot be validated */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (DataRegistryType)", CompactNodeTitle = "->", ScriptMethod = "ToString", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API FString Conv_DataRegistryTypeToString(FDataRegistryType DataRegistryType);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (DataRegistryType)", CompactNodeTitle = "==", ScriptOperator = "==", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API bool EqualEqual_DataRegistryType(FDataRegistryType A, FDataRegistryType B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (DataRegistryType)", CompactNodeTitle = "!=", ScriptOperator = "!=", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API bool NotEqual_DataRegistryType(FDataRegistryType A, FDataRegistryType B);

	/** Returns true if this is a non-empty item identifier, does not check if it is currently registered */
	UFUNCTION(BlueprintPure, Category = "AssetManager", meta=(ScriptMethod="IsValid", ScriptOperator="bool", BlueprintThreadSafe))
	static UE_API bool IsValidDataRegistryId(FDataRegistryId DataRegistryId);

	/** Converts a Data Registry Id to a string. The other direction is not provided because it cannot be validated */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (DataRegistryId)", CompactNodeTitle = "->", ScriptMethod="ToString", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API FString Conv_DataRegistryIdToString(FDataRegistryId DataRegistryId);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (DataRegistryId)", CompactNodeTitle = "==", ScriptOperator="==", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API bool EqualEqual_DataRegistryId(FDataRegistryId A, FDataRegistryId B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (DataRegistryId)", CompactNodeTitle = "!=", ScriptOperator="!=", BlueprintThreadSafe), Category = DataRegistry)
	static UE_API bool NotEqual_DataRegistryId(FDataRegistryId A, FDataRegistryId B);


	// Native interface, works using subsystem instance

	/** Returns the global subsystem instance, this can return null during early engine startup and shutdown */
	static UE_API UDataRegistrySubsystem* Get();

	/** Finds the right registry for a type name */
	UE_API UDataRegistry* GetRegistryForType(FName RegistryType) const;

	/** Returns proper display text for an id, using the correct id format */
	UE_API FText GetDisplayTextForId(FDataRegistryId ItemId) const;

	/** Gets list of all registries, useful for iterating in UI or utilities */
	UE_API void GetAllRegistries(TArray<UDataRegistry*>& AllRegistries, bool bSortByType = true) const;

	/** Refreshes the active registry map based on what's in memory */
	UE_API void RefreshRegistryMap();

	/** Loads all registry assets and initializes them, this is called early in startup */
	UE_API void LoadAllRegistries();

	/** True if all registries should have been initialized*/
	UE_API bool AreRegistriesInitialized() const;

	/** Returns true if the system is enabled via any config scan settings, will optionally warn if not enabled */
	UE_API bool IsConfigEnabled(bool bWarnIfNotEnabled = false) const;

	/** Returns true if the system is ready for initialization, if it hasn't started already */
	UE_API bool IsReadyForInitialization() const;

	/** Accesses the delegate called when the subsystem has finished scanning for and initializing all known data registries */
	UE_API FDataRegistrySubsystemInitializedCallback& OnSubsystemInitialized();

	/** Accesses the delegate called before the subsystem has loaded data registries */
	UE_API FPreLoadAllDataRegistriesCallback& OnPreLoadAllDataRegistries();

	/** Initializes all loaded registries and prepares them for queries */
	UE_API void InitializeAllRegistries(bool bResetIfInitialized = false);

	/** De-initializes all loaded registries */
	UE_API void DeinitializeAllRegistries();

	/** Load and initialize a specific registry, useful for plugins. This can hitch so the asset should be preloaded elsewhere if needed */
	UE_API bool LoadRegistryPath(const FSoftObjectPath& RegistryAssetPath);

	/** Removes specific data registry asset from the registration map, can be undone with LoadRegistryPath */
	UE_API bool IgnoreRegistryPath(const FSoftObjectPath& RegistryAssetPath);

	/** Resets state for all registries, call when gameplay has concluded to destroy caches */
	UE_API void ResetRuntimeState();

	/** Handles changes to DataRegistrySettings while engine is running */
	UE_API void ReinitializeFromConfig();

	/** 
	 * Attempt to register a specified asset with all active sources that allow dynamic registration, returning true if anything changed.
	 * This will fail if the registry does not exist yet.
	 *
	 * @param RegistryType		Type to register with, if invalid will try all registries
	 * @param AssetData			Filled in asset data of asset to attempt to register
	 * @Param AssetPriority		Priority of asset relative to others, higher numbers will be searched first
	 */
	UE_API bool RegisterSpecificAsset(FDataRegistryType RegistryType, FAssetData& AssetData, int32 AssetPriority = 0);

	/** Removes references to a specific asset, returns bool if it was removed */
	UE_API bool UnregisterSpecificAsset(FDataRegistryType RegistryType, const FSoftObjectPath& AssetPath);

	/** Unregisters all previously registered assets in a specific registry with a specific priority, can be used as a batch reset. Returns number of assets unregistered */
	UE_API int32 UnregisterAssetsWithPriority(FDataRegistryType RegistryType, int32 AssetPriority);

	/** Schedules registration of assets by path, this will happen immediately or will be queued if the data registries don't exist yet */
	UE_API void PreregisterSpecificAssets(const TMap<FDataRegistryType, TArray<FSoftObjectPath>>& AssetMap, int32 AssetPriority = 0);

	/** Gets the cached or precached data and struct type. The return value specifies the cache safety for the data */
	UE_API FDataRegistryCacheGetResult GetCachedItemRaw(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId) const;

	/** Gets the cached or precached data and struct type using an async acquire result. The return value specifies the cache safety for the data */
	UE_API FDataRegistryCacheGetResult GetCachedItemRawFromLookup(const uint8*& OutItemMemory, const UScriptStruct*& OutItemStruct, const FDataRegistryId& ItemId, const FDataRegistryLookup& Lookup) const;

	/** Computes an evaluated curve value, as well as the actual curve if it is found. The return value specifies the cache safety for the curve */
	UE_API FDataRegistryCacheGetResult EvaluateCachedCurve(float& OutValue, const FRealCurve*& OutCurve, FDataRegistryId ItemId, float InputValue, float DefaultValue = 0.0f) const;

	/** Returns a cached item of specified struct type. This will return null if the item is not already in memory */
	template <class T>
	const T* GetCachedItem(const FDataRegistryId& ItemId) const
	{
		const UDataRegistry* FoundRegistry = GetRegistryForType(ItemId.RegistryType);
		if (FoundRegistry)
		{
			return FoundRegistry->GetCachedItem<T>(ItemId);
		}
		return nullptr;
	}

	/** Start an async load of an item, delegate will be called on success or failure of acquire. Returns false if delegate could not be scheduled */
	UE_API bool AcquireItem(const FDataRegistryId& ItemId, FDataRegistryItemAcquiredCallback DelegateToCall) const;


	// Debug commands, bound as cvars or callable manually

	/** Outputs all registered types and some info */
	static UE_API void DumpRegistryTypeSummary();

	/** Dumps out a text representation of every item in the registry */
	static UE_API void DumpCachedItems(const TArray<FString>& Args);

#if WITH_EDITOR
	/** Returns true if the system will wait until first access to process pending Data Registry loads */
	UE_API bool WantsDelayedDataRegistryLoadingUntilPIE() const;
#endif

protected:
	typedef TPair<FName, TObjectPtr<UDataRegistry>> FRegistryMapPair;
	TSortedMap<FName, TObjectPtr<UDataRegistry>, FDefaultAllocator, FNameFastLess> RegistryMap;

	// Initialization order, need to wait for other early-load systems to initialize
	UE_API virtual void PostEngineInit();
	UE_API virtual void PostGameplayTags();
	UE_API virtual void PostAssetManager();
	UE_API virtual void ApplyPreregisterMap(UDataRegistry* Registry);

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// Paths that will be scanned for registries
	TArray<FString> AssetScanPaths;

	// Specific registries to load, will be added to AssetScanPaths at scan time
	TArray<FSoftObjectPath> RegistryPathsToLoad;

	// Specific registries to avoid registering, may be in memory but will not be registered
	TArray<FSoftObjectPath> RegistryPathsToIgnore;

	// List of assets to attempt to register when data registries come online
	typedef TPair<FSoftObjectPath, int32> FPreregisterAsset;
	TMap<FDataRegistryType, TArray<FPreregisterAsset>> PreregisterAssetMap;

	// True if initialization has finished and registries were scanned, will be false if not config enabled
	bool bFullyInitialized = false;

	// True if initialization is ready to start, will be true even if config disabled
	bool bReadyForInitialization = false;

	/** Callback for when the subsystem has finished scanning for and initializing all known data registries */
	FDataRegistrySubsystemInitializedCallback OnSubsystemInitializedCallback;

	/** Callback for before the subsystem has loaded data registries */
	FPreLoadAllDataRegistriesCallback OnPreLoadAllDataRegistriesCallback;

	/** Singleton object for the DataRegistrySubsystem::Get function to use, populated in Initialize and cleared out in Deinitialize */
	static UE_API TObjectPtr<UDataRegistrySubsystem> SingletonSubSystem;

#if WITH_EDITOR
	bool bLoadAllRegistriesOnNextPIE = false;
	UE_API virtual void PreBeginPIE(bool bStartSimulate);
	UE_API virtual void EndPIE(bool bStartSimulate);
#endif
	UE_API void FlushDeferredRegistryLoad(bool bExpected) const;
};

/* Test actor, move later
UCLASS(Blueprintable)
class ADataRegistryTestActor : public AActor
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category=DataRegistry)
	bool TestSyncRead(FDataRegistryId RegistryId);

	UFUNCTION(BlueprintCallable, Category = DataRegistry)
	bool TestAsyncRead(FDataRegistryId RegistryId);

	void AsyncReadComplete(const FDataRegistryAcquireResult& Result);
};
*/

#undef UE_API
