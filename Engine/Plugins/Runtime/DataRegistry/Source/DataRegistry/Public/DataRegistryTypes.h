// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryId.h"
#include "GameplayTagContainer.h"
#include "DataRegistryTypes.generated.h"

#define UE_API DATAREGISTRY_API


/** General rule about how hard it is to access an item, with later entries being the most available and faster to access */
UENUM()
enum class EDataRegistryAvailability : uint8
{
	/** Item definitely does not exist */
	DoesNotExist,

	/** Not sure where item is located or if it exists at all */
	Unknown,

	/** From a database or website with very high latency */
	Remote,

	/** From some other asset such as a json file available without internet access */
	OnDisk,

	/** Comes from a local asset, can be sync loaded as needed */
	LocalAsset,

	/** This item has already been loaded into memory by a different system and is immediately available */
	PreCached,
};


/** Struct representing how a unique id is formatted and picked in the editor */
USTRUCT()
struct FDataRegistryIdFormat
{
	GENERATED_BODY()

	/** If this is not empty, all ids are part of this hierarchy */
	UPROPERTY(EditAnywhere, Category=Default)
	FGameplayTag BaseGameplayTag;

	/** If this is not empty, all ids are actually primary assets of this type */
	//UPROPERTY(EditAnywhere, Category = Default)
	//FPrimaryAssetType BaseAssetType;
};


/** Rules to use when deciding how to unload registry items and related assets */
USTRUCT(BlueprintType)
struct FDataRegistryCachePolicy
{
	GENERATED_BODY()

	/** If this is true, the cache is always considered volatile when returning EDataRegistryCacheResult */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	bool bCacheIsAlwaysVolatile = false;

	/** If this is true, the cache version is synchronized with the global CurveTable cache version */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	bool bUseCurveTableCacheVersion = false;

	/** Will not release items if fewer then this number loaded, 0 means infinite */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	int32 MinNumberKept = 0;
	
	/** Maximum number of items to keep loaded, 0 means infinite */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	int32 MaxNumberKept = 0;

	/** Any item accessed within this amount of seconds is never unloaded */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	float ForceKeepSeconds = 0.0;

	/** Any item not accessed within this amount of seconds is always unloaded */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Cache)
	float ForceReleaseSeconds = 0.0;
};


/** A resolved unique id for a specific source, explains how to look it up. This type is opaque in blueprints and cannot be saved across runs */
USTRUCT(BlueprintType)
struct FDataRegistryLookup
{
	GENERATED_BODY()

	/** List of sources and names to look up within that source */
	// TODO can be more efficient, probably best option is pack it into the num field of the FName, no one really names entries Foo_2147483646
	TArray<TTuple<uint8, FName>, TInlineAllocator<1>> SourceLookups;

	/** Resets to default */
	inline void Reset()
	{
		SourceLookups.Reset();
	}

	/** Adds a new entry to lookup */
	inline void AddEntry(uint8 Index, FName ResolvedName)
	{
		SourceLookups.Emplace(Index, ResolvedName);
	}

	/** True if this lookup could work */
	inline bool IsValid() const
	{
		return SourceLookups.Num() > 0;
	}

	/** Get number of lookups */
	inline int32 GetNum() const
	{
		return SourceLookups.Num();
	}

	/** Gets a name and num */
	inline bool GetEntry(uint8& OutIndex, FName& OutResolvedName, int32 NumEntry) const
	{
		if (SourceLookups.IsValidIndex(NumEntry))
		{
			OutIndex = SourceLookups[NumEntry].Get<0>();
			OutResolvedName = SourceLookups[NumEntry].Get<1>();
			return true;
		}
		return false;
	}

	/** Generate a debug string */
	UE_API FString ToString() const;

	friend inline uint32 GetTypeHash(const FDataRegistryLookup& Lookup)
	{
		uint32 Hash = GetTypeHash(Lookup.SourceLookups.Num());
		for (int32 i = 0; i < Lookup.SourceLookups.Num(); i++)
		{
			Hash = HashCombine(Hash, GetTypeHash(Lookup.SourceLookups[i]));
		}
		return Hash;
	}

	friend inline bool operator==(const FDataRegistryLookup& X, const FDataRegistryLookup& Y)
	{
		return X.SourceLookups == Y.SourceLookups;
	}

	friend inline bool operator!=(const FDataRegistryLookup& X, const FDataRegistryLookup& Y)
	{
		return X.SourceLookups != Y.SourceLookups;
	}
};

template<>
struct TStructOpsTypeTraits<FDataRegistryLookup> : public TStructOpsTypeTraitsBase2<FDataRegistryLookup>
{
	enum
	{
		WithCopy = true, // This ensures the opaque type is copied correctly in BPs
	};
};


/** A debugging/editor struct used to describe the source for a single data registry item */
USTRUCT()
struct FDataRegistrySourceItemId
{
	GENERATED_BODY()

	/** Id that can be used to look this up by default */
	FDataRegistryId ItemId;

	/** Lookup used for caching */
	FDataRegistryLookup CacheLookup;

	/** Name this resolves to in the appropriate source */
	FName SourceResolvedName;

	/** Cached pointer to which source it came from, if this is invalid then cache lookup will fail */
	TWeakObjectPtr<const class UDataRegistrySource> CachedSource;

	/** Gets a unique ID key for this item, or None for invalid cached data */
	UE_API FString GetDebugString() const;

	friend inline uint32 GetTypeHash(const FDataRegistrySourceItemId& SourceItem)
	{
		uint32 Hash = GetTypeHash(SourceItem.ItemId);
		Hash = HashCombine(Hash, GetTypeHash(SourceItem.CacheLookup));
		Hash = HashCombine(Hash, GetTypeHash(SourceItem.SourceResolvedName));
		return Hash;
	}

	friend inline bool operator==(const FDataRegistrySourceItemId& X, const FDataRegistrySourceItemId& Y)
	{
		return X.ItemId == Y.ItemId && X.CacheLookup == Y.CacheLookup && X.SourceResolvedName == Y.SourceResolvedName;
	}

	friend inline bool operator!=(const FDataRegistrySourceItemId& X, const FDataRegistrySourceItemId& Y)
	{
		return !(X.ItemId == Y.ItemId && X.CacheLookup == Y.CacheLookup && X.SourceResolvedName == Y.SourceResolvedName);
	}
};


/** Information about the cache status of an item */
enum class EDataRegistryCacheGetStatus : uint8
{
	/** Item was not found in the cache */
	NotFound,

	/** Item was found, but is likely to change if requested again. This is never safe to cache outside the function stack */
	FoundVolatile,

	/** Item was found, and this will be safe to cache as long as the GetCacheVersion on the DataRegistry has not changed */
	FoundPersistent,
};

/** Error code returned when attempting to register a specific asset. */
enum class EDataRegistryRegisterAssetResult: uint8
{
	/** Asset was not registered */
	NotRegistered,

	/** Asset already registered in the data registry*/
	AssetAlreadyRegistered,

	/** Asset was successfully registered */
	RegisteredSuccesfully, // Maintain as highest result.
};

/** Where the cache version comes from, the upper values of this can be used for game-specific sources */
enum class EDataRegistryCacheVersionSource : uint8
{
	/** Invalid version */
	Invalid,

	/** Per registry version */
	DataRegistry,

	/** The global curve table version */
	CurveTable,

	/** Game-specific versions */
	GameVersion1,
	GameVersion2,
	GameVersion3,
	GameVersion4,
	GameVersion5,
	GameVersion6,
	GameVersion7,
	GameVersion8,
	GameVersion9,
	GameVersion10,
};

/** Information about success and usage safety of cache queries */
struct FDataRegistryCacheGetResult
{
	/** Default constructors, represents a failed get */
	FDataRegistryCacheGetResult() = default;

	/** Explicit constructor for successful get */
	FDataRegistryCacheGetResult(EDataRegistryCacheGetStatus InStatus, EDataRegistryCacheVersionSource InVersionSource = EDataRegistryCacheVersionSource::Invalid, int32 InCacheVersion = INDEX_NONE)
		: ItemStatus(InStatus)
		, VersionSource(InVersionSource)
		, CacheVersion(InCacheVersion)
	{}

	/** Return true if get was successful */
	inline bool WasFound() const
	{
		return (ItemStatus != EDataRegistryCacheGetStatus::NotFound);
	}

	/** Returns true if this cached value is safe to persist */
	inline bool IsPersistent() const
	{
		return (ItemStatus == EDataRegistryCacheGetStatus::FoundPersistent);
	}

	/** Returns true if this cached value is safe to use given the version number */
	inline bool IsValidForVersion(EDataRegistryCacheVersionSource InVersionSource, int32 InVersion) const
	{
		return (IsPersistent()) && (InVersionSource == VersionSource) && (InVersion == CacheVersion);
	}

	/** Returns the success/failure status */
	inline EDataRegistryCacheGetStatus GetItemStatus() const
	{
		return ItemStatus;
	}

	/** Returns the version source, which is used to validate */
	inline EDataRegistryCacheVersionSource GetVersionSource() const
	{
		return VersionSource;
	}

	/** Returns the cache version number */
	inline int32 GetCacheVersion() const
	{
		return CacheVersion;
	}

	// Operators

	FDataRegistryCacheGetResult(const FDataRegistryCacheGetResult& Other) = default;
	FDataRegistryCacheGetResult& operator=(const FDataRegistryCacheGetResult& Other) = default;

	inline explicit operator bool() const
	{
		return WasFound();
	}

	inline bool operator !() const
	{
		return !WasFound();
	}

private:
	// Private to avoid modification outside registry system

	/** Status of item inside cache */
	EDataRegistryCacheGetStatus ItemStatus = EDataRegistryCacheGetStatus::NotFound;

	/** Status of item inside cache */
	EDataRegistryCacheVersionSource VersionSource = EDataRegistryCacheVersionSource::Invalid;

	/** For persistent cache results, the version number of the registry at cache retrieval time */
	int32 CacheVersion = INDEX_NONE;

};


/** State of a registry async request */
UENUM()
enum class EDataRegistryAcquireStatus : uint8
{
	/** Not started yet */
	NotStarted,

	/** Initial acquire still in progress */
	WaitingForInitialAcquire,

	/** Temporary state, finished acquiring data but need to check resources */
	InitialAcquireFinished,

	/** Data requested and returned, still loading dependent resources */
	WaitingForResources,

	/** Fully loaded */
	AcquireFinished,

	/** Failed to acquire, may have timed out or had network issues, can be retried later */
	AcquireError,

	/** Known to not exist, cannot be retried */
	DoesNotExist,
};

/** Result struct for acquiring, this should never be stored long term and the memory is only valid in the current stack frame */
struct FDataRegistryAcquireResult
{
	/** It is not safe to copy or store this struct as the memory may not be valid outside this stack frame */
	FDataRegistryAcquireResult(const FDataRegistryAcquireResult&) = delete;
	FDataRegistryAcquireResult& operator=(const FDataRegistryAcquireResult&) = delete;

	FDataRegistryAcquireResult(const FDataRegistryId& InItemId, const FDataRegistryLookup& InResolvedLookup, EDataRegistryAcquireStatus InStatus, const UScriptStruct* InItemStruct = nullptr, const uint8* inItemMemory = nullptr)
		: ItemId(InItemId)
		, ResolvedLookup(InResolvedLookup)
		, Status(InStatus)
		, ItemStruct(InItemStruct)
		, ItemMemory(inItemMemory)
	{}

	/** The identifier this item corresponds to, the same struct can have multiple ids to refer to it */
	FDataRegistryId ItemId;

	/** The lookup this was resolved to, can use this to get item out of cache later without re-resolving */
	FDataRegistryLookup ResolvedLookup;

	/** Result of query */
	EDataRegistryAcquireStatus Status = EDataRegistryAcquireStatus::NotStarted;

	/** Type of the data acquired */
	const UScriptStruct* ItemStruct = nullptr;

	/** Memory of struct, this will either be null, or a totally valid struct */
	const uint8* ItemMemory = nullptr;

	/** Checks type and returns valid struct if it exists */
	template <class T>
	const T* GetItem() const
	{
		if (!ItemStruct || !ItemMemory)
		{
			return nullptr;
		}

		if (!ensureMsgf(ItemStruct->IsChildOf(T::StaticStruct()), TEXT("Can't cast data item of type %s to %s! Code should check type before calling GetItem"), *ItemStruct->GetName(), *T::StaticStruct()->GetName()))
		{
			return nullptr;
		}

		return reinterpret_cast<const T*>(ItemMemory);
	}

	/** Modifies acquire status based on new information, will advance status but not override error results */
	static UE_API void UpdateAcquireStatus(EDataRegistryAcquireStatus& CurrentStatus, EDataRegistryAcquireStatus NewStatus);
};


/** Simple struct describing an acquire request, can be subclassed by different sources */
struct FDataRegistrySourceAcquireRequest
{
	/** What lookup we are acquiring for */
	FDataRegistryLookup Lookup;

	/** Which source in lookup this is for */
	int32 LookupIndex = -1;
};


/** Simple unique id for an acquire request */
struct FDataRegistryRequestId
{
	/** Returns true if this is a real request */
	bool IsValid() const
	{
		return RequestId != InvalidId;
	}

	/** Gets an id to add request */
	static FDataRegistryRequestId GetNewRequestId();

	bool operator==(const FDataRegistryRequestId& Other) const
	{
		return RequestId == Other.RequestId;
	}

	bool operator!=(const FDataRegistryRequestId& Other) const
	{
		return RequestId != Other.RequestId;
	}

	bool operator<(const FDataRegistryRequestId& Other) const
	{
		return RequestId < Other.RequestId;
	}

	friend inline uint32 GetTypeHash(const FDataRegistryRequestId& Key)
	{
		return GetTypeHash(Key.RequestId);
	}

private:
	static const int32 InvalidId = 0;

	int32 RequestId = InvalidId;
};


/** Abstract structure used to resolve data registry IDs by game or plugin-specific systems */
struct FDataRegistryResolver
{
	/** Override this function, if it returns true then OutResolvedName should be used, otherwise will check other resolvers and default behavior */
	virtual bool ResolveIdToName(FName& OutResolvedName, const FDataRegistryId& ItemId, const class UDataRegistry* Registry, const class UDataRegistrySource* RegistrySource) = 0;

	/** Return true if this resolver is considered volatile, which means that it should stop any long term caching of results. This should be true for most stack-based resolvers */
	virtual bool IsVolatile() { return true; }
};

/** Scope object to set a temporary resolver, or register a global one */
struct FDataRegistryResolverScope
{
	/** Creates a temporary resolver scope without requiring a dynamically allocated object. Only valid on game thread */
	UE_API FDataRegistryResolverScope(FDataRegistryResolver& ScopeResolver);

	/** Creates a temporary resolver scope, will use the passed in resolver until scope returns. Only valid on game thread */
	UE_API FDataRegistryResolverScope(const TSharedPtr<FDataRegistryResolver>& ScopeResolver);

	/** Clears scope */
	UE_API ~FDataRegistryResolverScope();

	/** Static function to register a global scope */
	static UE_API void RegisterGlobalResolver(const TSharedPtr<FDataRegistryResolver>& ScopeResolver);

	/** Unregister a global scope, must have been registered already */
	static UE_API void UnregisterGlobalResolver(const TSharedPtr<FDataRegistryResolver>& ScopeResolver);

	/** Use the stack to resolve an ID, will return the resolver as raw pointer if found */
	static UE_API FDataRegistryResolver* ResolveNameFromId(FName& OutResolvedName, const FDataRegistryId& ItemId, const class UDataRegistry* Registry, const class UDataRegistrySource* RegistrySource);

	/** Returns true if there are any volatile resolvers on the stack */
	static UE_API bool IsStackVolatile();

private:
	// Stack depth at point this was added
	FDataRegistryResolver* AddedScopeResolver;

	// Global and temporary resolver stack, will go from last added back to 0, optionally able to push stack objects
	struct FResolverStackEntry
	{
		FResolverStackEntry(const TSharedPtr<FDataRegistryResolver>& ScopeResolver)
			: AsSharedPtr(ScopeResolver)
			, AsRawPtr(ScopeResolver.Get())
		{
		}

		FResolverStackEntry(FDataRegistryResolver& ScopeResolver)
			: AsSharedPtr()
			, AsRawPtr(&ScopeResolver)
		{
		}

		TSharedPtr<FDataRegistryResolver> AsSharedPtr;
		FDataRegistryResolver* AsRawPtr;
	};

	static TArray<FResolverStackEntry> ResolverStack;

};


/** Delegate called in C++ when item is loaded */
DECLARE_DELEGATE_OneParam(FDataRegistryItemAcquiredCallback, const FDataRegistryAcquireResult&);

/** Blueprint delegate called when item is loaded, you will need to re-query the cache */
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FDataRegistryItemAcquiredBPCallback, FDataRegistryId, ItemId, FDataRegistryLookup, ResolvedLookup, EDataRegistryAcquireStatus, Status);

/** Delegate called in C++ when multiple items are acquired, returns worst error experienced */
DECLARE_DELEGATE_OneParam(FDataRegistryBatchAcquireCallback, EDataRegistryAcquireStatus);

/** Multicast delegate broadcast called when a data registry's cache version has changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FDataRegistryCacheVersionCallback, class UDataRegistry*);

/** Multicast delegate broadcast called when the data registry subsystem has finished scanning for and initializing all known data registries */
DECLARE_MULTICAST_DELEGATE(FDataRegistrySubsystemInitializedCallback);

/** Multicast delegate broadcast called before the data registry subsystem loads all registries */
DECLARE_MULTICAST_DELEGATE(FPreLoadAllDataRegistriesCallback);


DATAREGISTRY_API DECLARE_LOG_CATEGORY_EXTERN(LogDataRegistry, Log, All);

#undef UE_API
