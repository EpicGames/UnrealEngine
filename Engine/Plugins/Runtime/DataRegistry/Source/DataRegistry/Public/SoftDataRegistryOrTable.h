// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistry.h"
#include "DataRegistryTypes.h"
#include "Engine/DataTable.h"
#include "Engine/StreamableManager.h"

#include "SoftDataRegistryOrTable.generated.h"

/** 
 * Defines a DataRegistry or a DataTable with a common interface to both
 */
USTRUCT(BlueprintType)
struct FSoftDataRegistryOrTable
{
	GENERATED_BODY()

	DATAREGISTRY_API FSoftDataRegistryOrTable();
	DATAREGISTRY_API FSoftDataRegistryOrTable(const UDataTable* InDataTable, const FDataRegistryType& InRegistryType);

	DATAREGISTRY_API bool Serialize(FArchive& Ar);

	/* Method to check if the given table matches the value in this struct */
	DATAREGISTRY_API bool Matches(const UDataTable* InTable) const;

	/* Method to check if the given registry matches the value in this struct */
	DATAREGISTRY_API bool Matches(const UDataRegistry* InRegistry) const;

	/* Method to check if the given FDataRegistryOrTableRow matches the value in this struct */
	DATAREGISTRY_API bool Matches(const FDataRegistryOrTableRow& RegistryOrTableId) const;

	/* Method to check validity of this struct (Checks that the registry or data table is set)*/
	DATAREGISTRY_API bool IsValid() const;

	template <class T>
	void ForEachItem(const FString& ContextString, TFunctionRef<void(const FName& Name, const T& Item)> Predicate) const
	{
		if (bUseDataRegistry)
		{
			const UDataRegistry* DataRegistry = GetDataRegistry();
			if (!DataRegistry)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found (%ls)  Registry:%ls", __FUNCTION__, *ContextString, *RegistryType.GetName().ToString());
				return;
			}

			DataRegistry->ForEachCachedItem<T>(ContextString, Predicate);
		}
		else
		{
			if (!Table)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Data Table found (%ls)", __FUNCTION__, *ContextString);
				return;
			}

			Table->ForeachRow<T>(ContextString, Predicate);
		}
	}

	/* This function returns an array of all items in the given registry or data table */
	template <class T>
	void GetItems(const TCHAR* ContextString, TArray<const T*>& Items) const
	{
		if (bUseDataRegistry)
		{
			const UDataRegistry* DataRegistry = GetDataRegistry();
			if (!DataRegistry)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found (%ls)  Registry:%ls", __FUNCTION__, ContextString, *RegistryType.GetName().ToString());
				return;
			}

			DataRegistry->GetAllItems(ContextString, Items);
		}
		else
		{
			if (!Table)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Data Table found (%ls)", __FUNCTION__, ContextString);
				return;
			}

			Table->GetAllRows<const T>(ContextString, Items);
		}
	}

	/* This function returns an array of all items in the given registry or data table */
	template <class T>
	void GetAllItems(const FString& ContextString, TArray<T*>& Items) const
	{
		GetItems<T>(*ContextString, Items);
	}

	/* This function returns an array of all item names in the given registry or data table */
	void GetItemNames(TArray<FName>& ItemNames) const
	{
		if (bUseDataRegistry)
		{
			const UDataRegistry* DataRegistry = GetDataRegistry();
			if (!DataRegistry)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found  Registry:%ls", __FUNCTION__, *RegistryType.GetName().ToString());
				return;
			}

			DataRegistry->GetItemNames(ItemNames);
		}
		else
		{
			if (!Table)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Data Table found", __FUNCTION__);
				return;
			}

			ItemNames = Table->GetRowNames();
		}
	}

	FString GetName() const
	{
		if (bUseDataRegistry)
		{
			const UDataRegistry* DataRegistry = GetDataRegistry();
			if (DataRegistry)
			{
				return DataRegistry->GetName();
			}
		}

		if (Table)
		{
			return Table->GetName();
		}

		return TEXT("");
	}

	/** Used to upgrade a SoftObjectPtr to a FSoftDataRegistryOrTable */
	DATAREGISTRY_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** Returns whether or not the registry or table is loaded */
	DATAREGISTRY_API bool IsLoaded() const;

	/**
	 * Requests an async load of a data table using the StreamableManager and then execute the callback, which will happen even if the load fails.
	 * NOTE: Data registries cannot be loaded this way because they must be registered with the data registry subsystem.
	 * @param DelegateToCall		Delegate to call when load finishes. Will be called on the next tick if asset is already loaded, or many seconds later
	 */
	DATAREGISTRY_API void LoadAsync(FStreamableDelegate DelegateToCall);	

	/**
	 * Returns the script struct used by this data registry or table
	 */
	DATAREGISTRY_API const UScriptStruct* GetStruct() const;

	/** Method to get a FDataRegistryOrTableRow from the given row name */
	DATAREGISTRY_API FDataRegistryOrTableRow GetRegistryOrTableRow(FName RowName) const;

	UPROPERTY(EditAnywhere, Category = DataRegistryOrTable)
	bool bUseDataRegistry = false;

	/** Data Table */
	UPROPERTY(EditAnywhere, Category = DataRegistryOrTable, meta = (EditCondition = "!bUseDataRegistry", EditConditionHides))
	TSoftObjectPtr<UDataTable> Table;

	/** Data Registry */
	UPROPERTY(EditAnywhere, Category = DataRegistryOrTable, meta = (EditCondition = "bUseDataRegistry", EditConditionHides))
	FDataRegistryType RegistryType;

private:

	/* Method to get the Data Registry */
	DATAREGISTRY_API const UDataRegistry* GetDataRegistry() const;

};

template<>
struct TStructOpsTypeTraits<FSoftDataRegistryOrTable> : public TStructOpsTypeTraitsBase2<FSoftDataRegistryOrTable>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/** Defines a DataRegistryId or DataTableRowHandle with a common interface to both */
USTRUCT(BlueprintType)
struct FDataRegistryOrTableRow
{
	GENERATED_BODY()

	DATAREGISTRY_API FDataRegistryOrTableRow();
	DATAREGISTRY_API FDataRegistryOrTableRow(const FDataTableRowHandle& RowHandle);
	DATAREGISTRY_API FDataRegistryOrTableRow(const FDataRegistryId& RegistryId);

	/** Used to upgrade a FDataTableHandle to a FDataRegistryOrTableRow */
	DATAREGISTRY_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/* Method to get the Data Registry for the given registry id */
	DATAREGISTRY_API const UDataRegistry* GetDataRegistry() const;

	template <class T>
	const T* GetItem(const TCHAR* ContextString) const
	{
		if (bUseDataRegistryId)
		{
			const UDataRegistry* EventRegistry = GetDataRegistry();
			if (!EventRegistry)
			{
				UE_LOGF(LogDataRegistry, Warning, "[%s] No Registry found  Registry:%ls", __FUNCTION__, *DataRegistryId.RegistryType.GetName().ToString());
				return nullptr;
			}

			return EventRegistry->GetCachedItem<T>(DataRegistryId);
		}

		return DataTableRow.GetRow<T>(ContextString);
	}

	/**
	 * Returns the script struct used for the data registry item or data table row. Only works for data registries that
	 * are registered
	 */
	DATAREGISTRY_API const UScriptStruct* GetStruct() const;

	DATAREGISTRY_API FString ToString() const;

	DATAREGISTRY_API FName GetItemName() const;

	// method to check validity of this row
	DATAREGISTRY_API bool IsValid() const;

	UPROPERTY(EditAnywhere, Category = DataRegistryOrTable)
	bool bUseDataRegistryId = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataRegistryOrTable, meta = (EditCondition = "!bUseDataRegistryId", EditConditionHides))
	FDataTableRowHandle DataTableRow;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DataRegistryOrTable, meta = (EditCondition = "bUseDataRegistryId", EditConditionHides))
	FDataRegistryId DataRegistryId;
};

template<>
struct TStructOpsTypeTraits<FDataRegistryOrTableRow> : public TStructOpsTypeTraitsBase2<FDataRegistryOrTableRow>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

