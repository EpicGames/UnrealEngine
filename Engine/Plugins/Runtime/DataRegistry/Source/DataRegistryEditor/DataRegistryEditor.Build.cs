// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DataRegistryEditor : ModuleRules
	{
		public DataRegistryEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"DataRegistry",
					"GameplayTagsEditor",
					"DataTableEditor",
					"InputCore",
					"PropertyEditor",
					"Slate",
					"SlateCore",					
					"BlueprintGraph",
					"Kismet",
					"KismetCompiler",
					"GraphEditor",
					"UnrealEd",
					"WorkspaceMenuStructure",
					"ContentBrowser",
					"EditorWidgets",
					"ApplicationCore",
				}
			);
		}
	}
}
