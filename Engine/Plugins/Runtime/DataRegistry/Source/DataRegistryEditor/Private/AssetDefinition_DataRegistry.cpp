// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DataRegistry.h"
#include "DataRegistryEditorToolkit.h"

EAssetCommandResult UAssetDefinition_DataRegistry::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UDataRegistry* DataRegistry : OpenArgs.LoadObjects<UDataRegistry>())
	{
		FDataRegistryEditorToolkit::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, DataRegistry);
	}
	return EAssetCommandResult::Handled;
}
