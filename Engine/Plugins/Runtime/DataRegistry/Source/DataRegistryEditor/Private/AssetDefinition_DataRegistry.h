// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "DataRegistry.h"
#include "AssetDefinition_DataRegistry.generated.h"

UCLASS()
class UAssetDefinition_DataRegistry : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_DataRegistry", "Data Registry"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDataRegistry::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(140, 62, 35); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_DataRegistrySubMenu", "Data Table"), ECategoryMenuType::Section)
		};
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
