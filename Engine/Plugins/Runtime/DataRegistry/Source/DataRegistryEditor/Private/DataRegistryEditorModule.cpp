// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryEditorModule.h"
#include "DataRegistryIdCustomization.h"
#include "DataRegistryTypeCustomization.h"
#include "DataTableOrRegistryRowCustomization.h"
#include "SoftDataRegistryOrTableCustomization.h"
#include "DataRegistrySubsystem.h"
#include "EdGraphSchema_K2.h"
#include "PropertyCustomizationHelpers.h"
#include "EdGraphUtilities.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "DataRegistryEditor"

IMPLEMENT_MODULE(FDataRegistryEditorModule, DataRegistryEditor)

class FDataRegistryGraphPanelPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InPin->PinType.PinSubCategoryObject == FDataRegistryId::StaticStruct())
		{
			return SNew(SGraphPinStructInstance, InPin).StructEditWrapper(FDataRegistryIdEditWrapper::StaticStruct());
		}
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InPin->PinType.PinSubCategoryObject == FDataRegistryType::StaticStruct())
		{
			return SNew(SDataRegistryTypeGraphPin, InPin);
		}

		return nullptr;
	}
};

void FDataRegistryEditorModule::StartupModule()
{
	// Disable any UI feature if running in command mode
	if (!IsRunningCommandlet())
	{
		// Register customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout("DataRegistryId", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataRegistryIdCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("DataRegistryType", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataRegistryTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("DataRegistryOrTableRow", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataRegistryOrTableRowCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("SoftDataRegistryOrTable", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoftDataRegistryOrTableCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register Pins and nodes
		DataRegistryGraphPanelPinFactory = MakeShareable(new FDataRegistryGraphPanelPinFactory());
		FEdGraphUtilities::RegisterVisualPinFactory(DataRegistryGraphPanelPinFactory);
	}

	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
}

void FDataRegistryEditorModule::ShutdownModule()
{
	if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
	{
		if (DataRegistryGraphPanelPinFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualPinFactory(DataRegistryGraphPanelPinFactory);
		}

		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			PropertyModule.UnregisterCustomPropertyTypeLayout("SoftDataRegistryOrTable");
			PropertyModule.UnregisterCustomPropertyTypeLayout("DataRegistryOrTableRow");
			PropertyModule.UnregisterCustomPropertyTypeLayout("DataRegistryType");
			PropertyModule.UnregisterCustomPropertyTypeLayout("DataRegistryId");
		}
	}

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
}

TSharedRef<SWidget> FDataRegistryEditorModule::MakeDataRegistryTypeSelector(FOnGetDataRegistryDisplayText OnGetDisplayText, FOnSetDataRegistryType OnSetType, bool bAllowClear, FName FilterStructName)
{
	FOnGetPropertyComboBoxStrings GetStrings = FOnGetPropertyComboBoxStrings::CreateStatic(&FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings, bAllowClear, FilterStructName);
	FOnGetPropertyComboBoxValue GetValue = FOnGetPropertyComboBoxValue::CreateLambda([OnGetDisplayText]
		{
			return OnGetDisplayText.Execute().ToString();
		});
	FOnPropertyComboBoxValueSelected SetValue = FOnPropertyComboBoxValueSelected::CreateLambda([OnSetType](const FString& StringValue)
		{
			OnSetType.Execute(FName(*StringValue));
		});

	return PropertyCustomizationHelpers::MakePropertyComboBox(nullptr, GetStrings, GetValue, SetValue);
}

void FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems, bool bAllowClear, FName FilterStructName)
{
	UDataRegistrySubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();

	if (ensure(Subsystem))
	{
		TArray<UDataRegistry*> RegistryArray;
		Subsystem->GetAllRegistries(RegistryArray, true);

		// Can the field be cleared
		if (bAllowClear)
		{
			// Add None
			OutComboBoxStrings.Add(MakeShared<FString>(LOCTEXT("NoType", "None").ToString()));
			OutToolTips.Add(SNew(SToolTip).Text(LOCTEXT("NoType", "None")));
			OutRestrictedItems.Add(false);
		}

		for (UDataRegistry* Registry : RegistryArray)
		{
			if (Registry->DoesItemStructMatchFilter(FilterStructName))
			{
				OutComboBoxStrings.Add(MakeShared<FString>(Registry->GetRegistryType().ToString()));
				OutToolTips.Add(SNew(SToolTip).Text(Registry->GetRegistryDescription()));
				OutRestrictedItems.Add(false);
			}
		}
	}
}

TSharedRef<SWidget> FDataRegistryEditorModule::MakeDataRegistryItemNameSelector(FOnGetDataRegistryDisplayText OnGetDisplayText, FOnGetDataRegistryId OnGetId, FOnSetDataRegistryId OnSetId, FOnGetCustomDataRegistryItemNames OnGetCustomItemNames, bool bAllowClear /*= true*/)
{
	return SNew(SDataRegistryItemNameWidget)
		.OnGetDisplayText(OnGetDisplayText)
		.OnGetId(OnGetId)
		.OnSetId(OnSetId)
		.OnGetCustomItemNames(OnGetCustomItemNames)
		.bAllowClear(bAllowClear);
}

#undef LOCTEXT_NAMESPACE
