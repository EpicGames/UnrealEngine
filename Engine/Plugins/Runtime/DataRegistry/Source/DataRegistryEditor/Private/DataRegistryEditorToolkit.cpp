// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryEditorToolkit.h"
#include "DataRegistryEditorModule.h"
#include "DataRegistrySource.h"
#include "DetailsViewArgs.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "Math/ColorList.h"
#include "SDataRegistryListViewRow.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SHyperlink.h"
#include "SourceCodeNavigation.h"
#include "PropertyEditorModule.h"
#include "StructUtils/UserDefinedStruct.h"
 
#define LOCTEXT_NAMESPACE "DataRegistryEditor"

const FName FDataRegistryEditorToolkit::AppIdentifier("DataRegistryEditorApp");
const FName FDataRegistryEditorToolkit::RowListTabId("DataRegistryEditor_RowList");
const FName FDataRegistryEditorToolkit::PropertiesTabId("DataRegistryEditor_Properties");
const FName FDataRegistryEditorToolkit::RowNumberColumnId("RowNumber");
const FName FDataRegistryEditorToolkit::RowNameColumnId("RowName");
const FName FDataRegistryEditorToolkit::RowSourceColumnId("RowSource");
const FName FDataRegistryEditorToolkit::RowResolvedColumnId("RowResolved");

FDataRegistryEditorToolkit::FDataRegistryEditorToolkit()
	: RowNameColumnWidth(0)
	, RowNumberColumnWidth(0)
	, HighlightedVisibleRowIndex(INDEX_NONE)
	, SortMode(EColumnSortMode::Ascending)
{
}

FDataRegistryEditorToolkit::~FDataRegistryEditorToolkit()
{

}

TSharedRef<FDataRegistryEditorToolkit> FDataRegistryEditorToolkit::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UDataRegistry* InDataRegistry)
{
	TSharedRef<FDataRegistryEditorToolkit> NewEditor(new FDataRegistryEditorToolkit());
	NewEditor->InitDataRegistryEditor(Mode, InitToolkitHost, InDataRegistry);
	return NewEditor;
}

FName FDataRegistryEditorToolkit::GetToolkitFName() const
{
	return FName("DataRegistryEditor");
}

FText FDataRegistryEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "DataRegistry Editor");
}

FString FDataRegistryEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "DataRegistry ").ToString();
}

FLinearColor FDataRegistryEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}

void FDataRegistryEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_DataRegistryEditor", "Data Registry Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	CreateAndRegisterRowListTab(InTabManager);
	CreateAndRegisterPropertiesTab(InTabManager);
}

void FDataRegistryEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(RowListTabId);
	InTabManager->UnregisterTabSpawner(PropertiesTabId);

	RowListTabWidget.Reset();
	DetailsView.Reset();
}

void FDataRegistryEditorToolkit::CreateAndRegisterRowListTab(const TSharedRef<class FTabManager>& InTabManager)
{
	RowListTabWidget = CreateContentBox();

	InTabManager->RegisterTabSpawner(RowListTabId, FOnSpawnTab::CreateSP(this, &FDataRegistryEditorToolkit::SpawnTab_RowList))
		.SetDisplayName(LOCTEXT("RowListTab", "RowList"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FDataRegistryEditorToolkit::CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FDataRegistryEditorToolkit::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Properties"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FDataRegistryEditorToolkit::InitDataRegistryEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataRegistry* Registry)
{
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_DataRegistryEditor_Layout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.3f)	
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->SetHideTabWell(true)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(RowListTabId, ETabState::OpenedTab)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Registry);

	FDataRegistryEditorModule& DataRegistryEditorModule = FDataRegistryEditorModule::Get();
	AddMenuExtender(DataRegistryEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TSharedPtr<FExtender> ToolbarExtender = DataRegistryEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	ExtendToolbar(ToolbarExtender);

	AddToolbarExtender(ToolbarExtender);

	RegenerateMenusAndToolbars();

	if (DetailsView.IsValid())
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObject(GetEditingObject());
	}
}

const UDataRegistry* FDataRegistryEditorToolkit::GetDataRegistry() const
{
	return Cast<const UDataRegistry>(GetEditingObject());
}

UDataRegistry* FDataRegistryEditorToolkit::GetEditableDataRegistry() const
{
	return Cast<UDataRegistry>(GetEditingObject());
}

void FDataRegistryEditorToolkit::HandlePostChange()
{
	// We need to cache and restore the selection here as RefreshCachedDataRegistry will re-create the list view items
	const FName CachedSelection = HighlightedRowName;
	HighlightedRowName = NAME_None;
	RefreshCachedDataRegistry(CachedSelection, true/*bUpdateEvenIfValid*/);
}

void FDataRegistryEditorToolkit::OnCopyClicked()
{
	UDataRegistry* Registry = GetEditableDataRegistry();
	if (Registry)
	{
		CopySelectedRow();
	}
}

void FDataRegistryEditorToolkit::OnRefreshClicked()
{
	UDataRegistry* Registry = GetEditableDataRegistry();
	if (Registry)
	{
		Registry->EditorRefreshRegistry();
		HandlePostChange();
	}
}

void FDataRegistryEditorToolkit::OnDataAcquired(EDataRegistryAcquireStatus AcquireStatus)
{
	HandlePostChange();

	PendingSourceData.Empty();
}

void FDataRegistryEditorToolkit::SetDefaultSort()
{
	SortMode = EColumnSortMode::Ascending;
	SortByColumn = FDataRegistryEditorToolkit::RowNumberColumnId;
}

EColumnSortMode::Type FDataRegistryEditorToolkit::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void FDataRegistryEditorToolkit::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	int32 ColumnIndex;

	SortMode = InSortMode;
	SortByColumn = ColumnId;

	for (ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		if (AvailableColumns[ColumnIndex]->ColumnId == ColumnId)
		{
			break;
		}
	}

	if (AvailableColumns.IsValidIndex(ColumnIndex))
	{
		if (InSortMode == EColumnSortMode::Ascending)
		{
			VisibleRows.Sort([ColumnIndex](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
			{					
				int32 Result = (first->CellData[ColumnIndex].ToString()).Compare(second->CellData[ColumnIndex].ToString());

				if (!Result)
				{
					return first->RowNum < second->RowNum;

				}

				return Result < 0;
			});

		}
		else if (InSortMode == EColumnSortMode::Descending)
		{
			VisibleRows.Sort([ColumnIndex](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
			{
				int32 Result = (first->CellData[ColumnIndex].ToString()).Compare(second->CellData[ColumnIndex].ToString());

				if (!Result)
				{
					return first->RowNum > second->RowNum;
				}

				return Result > 0;
			});
		}
	}

	CellsListView->RequestListRefresh();
}

void FDataRegistryEditorToolkit::OnColumnNumberSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortMode = InSortMode;
	SortByColumn = ColumnId;

	if (InSortMode == EColumnSortMode::Ascending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return first->RowNum < second->RowNum;
		});
	}
	else if (InSortMode == EColumnSortMode::Descending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return first->RowNum > second->RowNum;
		});
	}

	CellsListView->RequestListRefresh();
}

void FDataRegistryEditorToolkit::OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortMode = InSortMode;
	SortByColumn = ColumnId;

	if (InSortMode == EColumnSortMode::Ascending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return (first->DisplayName).ToString() < (second->DisplayName).ToString();
		});
	}
	else if (InSortMode == EColumnSortMode::Descending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return (first->DisplayName).ToString() > (second->DisplayName).ToString();
		});
	}

	CellsListView->RequestListRefresh();
}

void FDataRegistryEditorToolkit::SetHighlightedRow(FName Name)
{
	if (Name == HighlightedRowName)
	{
		return;
	}

	if (Name.IsNone())
	{
		HighlightedRowName = NAME_None;
		CellsListView->ClearSelection();
		HighlightedVisibleRowIndex = INDEX_NONE;
	}
	else
	{
		HighlightedRowName = Name;

		FDataTableEditorRowListViewDataPtr* NewSelectionPtr = NULL;
		for (HighlightedVisibleRowIndex = 0; HighlightedVisibleRowIndex < VisibleRows.Num(); ++HighlightedVisibleRowIndex)
		{
			if (VisibleRows[HighlightedVisibleRowIndex]->RowId == Name)
			{
				NewSelectionPtr = &(VisibleRows[HighlightedVisibleRowIndex]);

				break;
			}
		}

		// Synchronize the list views
		if (NewSelectionPtr)
		{
			CellsListView->SetSelection(*NewSelectionPtr);

			CellsListView->RequestScrollIntoView(*NewSelectionPtr);
		}
		else
		{
			CellsListView->ClearSelection();
		}
	}
}

void FDataRegistryEditorToolkit::ExtendToolbar(TSharedPtr<FExtender> Extender)
{
	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FDataRegistryEditorToolkit::FillToolbar)
	);
}

void FDataRegistryEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("DataRegistryCommands");
	{
		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FDataRegistryEditorToolkit::OnRefreshClicked)),
			NAME_None,
			LOCTEXT("RefreshText", "Refresh"),
			LOCTEXT("RefreshTooltip", "Refresh the registry preview window"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.ReimportAsset"));

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FDataRegistryEditorToolkit::OnCopyClicked)),
			NAME_None,
			LOCTEXT("CopyIconText", "Copy"),
			LOCTEXT("CopyToolTip", "Copy the currently selected row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataTableEditor.Copy"));

	}
	ToolbarBuilder.EndSection();
}

FSlateColor FDataRegistryEditorToolkit::GetRowTextColor(FName RowName) const
{
	if (RowName == HighlightedRowName)
	{
		return FSlateColor(FColorList::Orange);
	}
	return FSlateColor::UseForeground();
}

const FDataRegistrySourceItemId* FDataRegistryEditorToolkit::GetSourceItemForName(FName DebugName) const
{
	return SourceItemMap.Find(DebugName);
}

FText FDataRegistryEditorToolkit::GetCellText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const
{
	if (InRowDataPointer.IsValid() && ColumnIndex < InRowDataPointer->CellData.Num())
	{
		return InRowDataPointer->CellData[ColumnIndex];
	}

	return FText();
}

FText FDataRegistryEditorToolkit::GetCellToolTipText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const
{
	FText TooltipText;

	if (ColumnIndex < AvailableColumns.Num())
	{
		TooltipText = AvailableColumns[ColumnIndex]->DisplayName;
	}

	if (InRowDataPointer.IsValid() && ColumnIndex < InRowDataPointer->CellData.Num())
	{
		TooltipText = FText::Format(LOCTEXT("ColumnRowNameFmt", "{0}: {1}"), TooltipText, InRowDataPointer->CellData[ColumnIndex]);
	}

	return TooltipText;
}

float FDataRegistryEditorToolkit::GetRowNumberColumnWidth() const
{
	return RowNumberColumnWidth;
}

void FDataRegistryEditorToolkit::RefreshRowNumberColumnWidth()
{

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	const float CellPadding = 10.0f;

	for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
	{
		const float RowNumberWidth = FontMeasure->Measure(FString::FromInt(RowData->RowNum), CellTextStyle.Font).X + CellPadding;
		RowNumberColumnWidth = FMath::Max(RowNumberColumnWidth, RowNumberWidth);
	}

}

void FDataRegistryEditorToolkit::OnRowNumberColumnResized(const float NewWidth)
{
	RowNumberColumnWidth = NewWidth;
}

float FDataRegistryEditorToolkit::GetRowNameColumnWidth() const
{
	return RowNameColumnWidth;
}

void FDataRegistryEditorToolkit::RefreshRowNameColumnWidth()
{

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	static const float CellPadding = 10.0f;

	for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
	{
		const float RowNameWidth = FontMeasure->Measure(RowData->DisplayName, CellTextStyle.Font).X + CellPadding;
		RowNameColumnWidth = FMath::Max(RowNameColumnWidth, RowNameWidth);
	}

}

float FDataRegistryEditorToolkit::GetColumnWidth(const int32 ColumnIndex) const
{
	if (ColumnWidths.IsValidIndex(ColumnIndex))
	{
		return ColumnWidths[ColumnIndex].CurrentWidth;
	}
	return 0.0f;
}

void FDataRegistryEditorToolkit::OnColumnResized(const float NewWidth, const int32 ColumnIndex)
{
	if (ColumnWidths.IsValidIndex(ColumnIndex))
	{
		FColumnWidth& ColumnWidth = ColumnWidths[ColumnIndex];
		ColumnWidth.bIsAutoSized = false;
		ColumnWidth.CurrentWidth = NewWidth;
	}
}

void FDataRegistryEditorToolkit::OnRowNameColumnResized(const float NewWidth)
{
	RowNameColumnWidth = NewWidth;
}

TSharedRef<ITableRow> FDataRegistryEditorToolkit::MakeRowWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(SDataRegistryListViewRow, OwnerTable)
		.DataRegistryEditor(SharedThis(this))
		.RowDataPtr(InRowDataPtr);
}

void FDataRegistryEditorToolkit::OnRowSelectionChanged(FDataTableEditorRowListViewDataPtr InNewSelection, ESelectInfo::Type InSelectInfo)
{
	const bool bSelectionChanged = !InNewSelection.IsValid() || InNewSelection->RowId != HighlightedRowName;
	const FName NewRowName = (InNewSelection.IsValid()) ? InNewSelection->RowId : NAME_None;

	SetHighlightedRow(NewRowName);

	if (bSelectionChanged)
	{
		//CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
}

void FDataRegistryEditorToolkit::CopySelectedRow()
{
	const FDataRegistrySourceItemId* FoundSource = GetSourceItemForName(HighlightedRowName);
	UDataRegistry* Registry = Cast<UDataRegistry>(GetEditingObject());

	if (Registry && Registry->GetItemStruct() && FoundSource)
	{
		const uint8* FoundData = nullptr;
		const UScriptStruct* FoundStruct = nullptr;
		Registry->GetCachedItemRawFromLookup(FoundData, FoundStruct, FoundSource->ItemId, FoundSource->CacheLookup);

		if (FoundData && FoundStruct)
		{
			FString ClipboardValue;
			FoundStruct->ExportText(ClipboardValue, FoundData, FoundData, Registry, PPF_Copy, nullptr);

			FPlatformApplicationMisc::ClipboardCopy(*ClipboardValue);
		}
	}
}

FText FDataRegistryEditorToolkit::GetFilterText() const
{
	return ActiveFilterText;
}

void FDataRegistryEditorToolkit::OnFilterTextChanged(const FText& InFilterText)
{
	ActiveFilterText = InFilterText;
	UpdateVisibleRows();
}

void FDataRegistryEditorToolkit::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnCleared)
	{
		SearchBoxWidget->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
	}
}

void FDataRegistryEditorToolkit::PostRegenerateMenusAndToolbars()
{
	const UDataRegistry* DataRegistry = GetDataRegistry();
	if (DataRegistry)
	{
		const UUserDefinedStruct* UDS = Cast<const UUserDefinedStruct>(DataRegistry->GetItemStruct());

		// build and attach the menu overlay
		TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ShadowOffset(FVector2D::UnitVector)
				.Text(LOCTEXT("DataRegistryEditor_RowStructType", "Row Type: "))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate(this, &FDataRegistryEditorToolkit::OnEditDataTableStructClicked)
				.Text(FText::FromString(GetNameSafe(DataRegistry->GetItemStruct())))
				.ToolTipText(LOCTEXT("DataRegistryStructToolTip", "Open the struct used for each row in this data table"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FDataRegistryEditorToolkit::OnFindRowInContentBrowserClicked)
				.Visibility(UDS ? EVisibility::Visible : EVisibility::Collapsed)
				.ToolTipText(LOCTEXT("FindRowInCBToolTip", "Find struct in Content Browser"))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			];
	
		SetMenuOverlay(MenuOverlayBox);
	}
}


FReply FDataRegistryEditorToolkit::OnFindRowInContentBrowserClicked()
{
	const UDataRegistry* DataRegistry = GetDataRegistry();
	if (DataRegistry)
	{
		TArray<FAssetData> ObjectsToSync;
		ObjectsToSync.Add(FAssetData(DataRegistry->GetItemStruct()));
		GEditor->SyncBrowserToObjects(ObjectsToSync);
	}

	return FReply::Handled();
}

void FDataRegistryEditorToolkit::OnEditDataTableStructClicked()
{
	const UDataRegistry* DataRegistry = GetDataRegistry();
	if (DataRegistry && DataRegistry->GetItemStruct())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DataRegistry->GetItemStruct()->GetPathName());
		if (FSourceCodeNavigation::CanNavigateToStruct(DataRegistry->GetItemStruct()))
		{
			FSourceCodeNavigation::NavigateToStruct(DataRegistry->GetItemStruct());
		}
	}
}

void FDataRegistryEditorToolkit::RefreshCachedDataRegistry(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	UDataRegistry* Registry = GetEditableDataRegistry();
	TArray<FDataTableEditorColumnHeaderDataPtr> PreviousColumns = AvailableColumns;

	TArray<FDataRegistrySourceItemId> SourceItems;
	TMap<FName, uint8*> CachedItems;

	AvailableColumns.Empty();
	AvailableRows.Empty();
	SourceItemMap.Empty();
	bool bNeedsResolvedName = false;

	TArray<FDataRegistrySourceItemId> MissingData;

	if (Registry && Registry->GetItemStruct())
	{
		Registry->GetAllSourceItems(SourceItems);

		for (FDataRegistrySourceItemId& SourceItem : SourceItems)
		{
			FName ItemKey = FName(*SourceItem.GetDebugString());
			if (ItemKey != NAME_None)
			{
				const uint8* FoundData = nullptr;
				const UScriptStruct* FoundStruct = nullptr;
				Registry->GetCachedItemRawFromLookup(FoundData, FoundStruct, SourceItem.ItemId, SourceItem.CacheLookup);

				if (FoundData)
				{
					CachedItems.Add(ItemKey, const_cast<uint8*>(FoundData));
					SourceItemMap.Add(ItemKey, SourceItem);

					// If our resolved name is ever different than item name, add that column
					if (SourceItem.ItemId.ItemName != SourceItem.SourceResolvedName)
					{
						bNeedsResolvedName = true;
					}
				}
				else
				{
					MissingData.Add(SourceItem);
				}
			}
		}

		// If we're missing data and a request isn't in progress, ask again
		if (MissingData.Num() && PendingSourceData.Num() == 0)
		{
			PendingSourceData = MissingData;
			Registry->BatchAcquireSourceItems(MissingData, FDataRegistryBatchAcquireCallback::CreateSP(this, &FDataRegistryEditorToolkit::OnDataAcquired));
		}

		FDataTableEditorUtils::CacheDataForEditing(Registry->GetItemStruct(), CachedItems, AvailableColumns, AvailableRows);

		if (AvailableColumns.Num() > 0)
		{
			// Prepend the fake columns for source info
			TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
			static const float CellPadding = 10.0f;

			const int32 SourceColumnIdx = 0;
			FDataTableEditorColumnHeaderDataPtr SourceColumn = MakeShareable(new FDataTableEditorColumnHeaderData());
			SourceColumn->ColumnId = RowSourceColumnId;
			SourceColumn->DisplayName = LOCTEXT("ColumnNameSource", "Source");
			SourceColumn->Property = nullptr;
			SourceColumn->DesiredColumnWidth = FontMeasure->Measure(SourceColumn->DisplayName, CellTextStyle.Font).X + CellPadding;
			AvailableColumns.Insert(SourceColumn, SourceColumnIdx);

			const int32 ResolvedColumnIdx = 1;
			FDataTableEditorColumnHeaderDataPtr ResolvedColumn = nullptr;
			if (bNeedsResolvedName)
			{
				ResolvedColumn = MakeShareable(new FDataTableEditorColumnHeaderData());
				ResolvedColumn->ColumnId = RowResolvedColumnId;
				ResolvedColumn->DisplayName = LOCTEXT("ColumnNameResolved", "Resolved Name");
				ResolvedColumn->Property = nullptr;
				ResolvedColumn->DesiredColumnWidth = FontMeasure->Measure(ResolvedColumn->DisplayName, CellTextStyle.Font).X + CellPadding;
				AvailableColumns.Insert(ResolvedColumn, ResolvedColumnIdx);
			}

			// Fix up row data for source info
			for (FDataTableEditorRowListViewDataPtr RowPtr : AvailableRows)
			{
				const FDataRegistrySourceItemId* FoundSourceData = SourceItemMap.Find(RowPtr->RowId);
				if (RowPtr->CellData.Num() != AvailableColumns.Num() && FoundSourceData)
				{
					// Reset display name back
					RowPtr->DisplayName = FText::FromName(FoundSourceData->ItemId.ItemName);
				
					// Insert source data
					const UDataRegistrySource* ResolvedSource = FoundSourceData->CachedSource.Get();
					RowPtr->CellData.Insert(ResolvedSource ? FText::AsCultureInvariant(ResolvedSource->GetDebugString()) : FText::GetEmpty(), SourceColumnIdx);
					SourceColumn->DesiredColumnWidth = FMath::Max(SourceColumn->DesiredColumnWidth, FontMeasure->Measure(RowPtr->CellData[SourceColumnIdx], CellTextStyle.Font).X + CellPadding);

					if (bNeedsResolvedName)
					{
						// Insert resolved data
						RowPtr->CellData.Insert(FText::FromName(FoundSourceData->SourceResolvedName), ResolvedColumnIdx);
						ResolvedColumn->DesiredColumnWidth = FMath::Max(ResolvedColumn->DesiredColumnWidth, FontMeasure->Measure(RowPtr->CellData[ResolvedColumnIdx], CellTextStyle.Font).X + CellPadding);
					}
				}
			}
		}
	}

	// Update the desired width of the row names and numbers column
	// This prevents it growing or shrinking as you scroll the list view
	RefreshRowNumberColumnWidth();
	RefreshRowNameColumnWidth();

	// Setup the default auto-sized columns
	ColumnWidths.SetNum(AvailableColumns.Num());
	for (int32 ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];
		FColumnWidth& ColumnWidth = ColumnWidths[ColumnIndex];
		ColumnWidth.CurrentWidth = FMath::Clamp(ColumnData->DesiredColumnWidth, 10.0f, 400.0f); // Clamp auto-sized columns to a reasonable limit
	}

	if (PreviousColumns != AvailableColumns)
	{
		ColumnNamesHeaderRow->ClearColumns();

		ColumnNamesHeaderRow->AddColumn(
			SHeaderRow::Column(RowNumberColumnId)
			.SortMode(this, &FDataRegistryEditorToolkit::GetColumnSortMode, RowNumberColumnId)
			.OnSort(this, &FDataRegistryEditorToolkit::OnColumnNumberSortModeChanged)
			.ManualWidth(this, &FDataRegistryEditorToolkit::GetRowNumberColumnWidth)
			.OnWidthChanged(this, &FDataRegistryEditorToolkit::OnRowNumberColumnResized)
			[
				SNew(SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(FText::GetEmpty())
				]
			]

		);

		ColumnNamesHeaderRow->AddColumn(
			SHeaderRow::Column(RowNameColumnId)
			.DefaultLabel(LOCTEXT("DataRegistryRowName", "Item Name"))
			.ManualWidth(this, &FDataRegistryEditorToolkit::GetRowNameColumnWidth)
			.OnWidthChanged(this, &FDataRegistryEditorToolkit::OnRowNameColumnResized)
			.SortMode(this, &FDataRegistryEditorToolkit::GetColumnSortMode, RowNameColumnId)
			.OnSort(this, &FDataRegistryEditorToolkit::OnColumnNameSortModeChanged)
		);

		for (int32 ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
		{
			const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];

			ColumnNamesHeaderRow->AddColumn(
				SHeaderRow::Column(ColumnData->ColumnId)
				.DefaultLabel(ColumnData->DisplayName)
				.ManualWidth(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &FDataRegistryEditorToolkit::GetColumnWidth, ColumnIndex)))
				.OnWidthChanged(this, &FDataRegistryEditorToolkit::OnColumnResized, ColumnIndex)
				.SortMode(this, &FDataRegistryEditorToolkit::GetColumnSortMode, ColumnData->ColumnId)
				.OnSort(this, &FDataRegistryEditorToolkit::OnColumnSortModeChanged)
				[
					SNew(SBox)
					.Padding(FMargin(0, 4, 0, 4))
					.VAlign(VAlign_Fill)
					.ToolTip(IDocumentation::Get()->CreateToolTip(FDataTableEditorUtils::GetRowTypeInfoTooltipText(ColumnData), nullptr, *FDataTableEditorUtils::VariableTypesTooltipDocLink, FDataTableEditorUtils::GetRowTypeTooltipDocExcerptName(ColumnData)))
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(ColumnData->DisplayName)
					]
				]
			);
		}
	}

	UpdateVisibleRows(InCachedSelection, bUpdateEvenIfValid);

	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(Registry);
	}
}

void FDataRegistryEditorToolkit::UpdateVisibleRows(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	if (ActiveFilterText.IsEmptyOrWhitespace())
	{
		VisibleRows = AvailableRows;
	}
	else
	{
		VisibleRows.Empty(AvailableRows.Num());

		const FString& ActiveFilterString = ActiveFilterText.ToString();
		for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
		{
			bool bPassesFilter = false;

			if (RowData->DisplayName.ToString().Contains(ActiveFilterString))
			{
				bPassesFilter = true;
			}
			else
			{
				for (const FText& CellText : RowData->CellData)
				{
					if (CellText.ToString().Contains(ActiveFilterString))
					{
						bPassesFilter = true;
						break;
					}
				}
			}

			if (bPassesFilter)
			{
				VisibleRows.Add(RowData);
			}
		}
	}

	CellsListView->RequestListRefresh();
	RestoreCachedSelection(InCachedSelection, bUpdateEvenIfValid);
}

void FDataRegistryEditorToolkit::RestoreCachedSelection(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	// Validate the requested selection to see if it matches a known row
	bool bSelectedRowIsValid = false;
	if (!InCachedSelection.IsNone())
	{
		bSelectedRowIsValid = VisibleRows.ContainsByPredicate([&InCachedSelection](const FDataTableEditorRowListViewDataPtr& RowData) -> bool
		{
			return RowData->RowId == InCachedSelection;
		});
	}

	// Apply the new selection (if required)
	if (!bSelectedRowIsValid)
	{
		SetHighlightedRow((VisibleRows.Num() > 0) ? VisibleRows[0]->RowId : NAME_None);
		//CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
	else if (bUpdateEvenIfValid)
	{
		SetHighlightedRow(InCachedSelection);
		//CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
}

TSharedRef<SVerticalBox> FDataRegistryEditorToolkit::CreateContentBox()
{
	TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	ColumnNamesHeaderRow = SNew(SHeaderRow);

	CellsListView = SNew(SListView<FDataTableEditorRowListViewDataPtr>)
		.ListItemsSource(&VisibleRows)
		.HeaderRow(ColumnNamesHeaderRow)
		.OnGenerateRow(this, &FDataRegistryEditorToolkit::MakeRowWidget)
		.OnSelectionChanged(this, &FDataRegistryEditorToolkit::OnRowSelectionChanged)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.SelectionMode(ESelectionMode::Single)
		.AllowOverscroll(EAllowOverscroll::No);

	RefreshCachedDataRegistry();

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(SearchBoxWidget, SSearchBox)
				.InitialText(this, &FDataRegistryEditorToolkit::GetFilterText)
				.OnTextChanged(this, &FDataRegistryEditorToolkit::OnFilterTextChanged)
				.OnTextCommitted(this, &FDataRegistryEditorToolkit::OnFilterTextCommitted)
			]
		]
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+SScrollBox::Slot()
				[
					CellsListView.ToSharedRef()
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				HorizontalScrollBar
			]
		];
}

TSharedRef<SDockTab> FDataRegistryEditorToolkit::SpawnTab_RowList(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == RowListTabId);

	UDataRegistry* Table = Cast<UDataRegistry>(GetEditingObject());

	// Support undo/redo
	if (Table)
	{
		Table->SetFlags(RF_Transactional);
	}

	return SNew(SDockTab)
		.Label(LOCTEXT("RowListTitle", "Registry Preview"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				RowListTabWidget.ToSharedRef()
			]
		];
}

TSharedRef<SDockTab> FDataRegistryEditorToolkit::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("PropertiesTitle", "Properties"))
		.TabColorScale(GetTabColorScale())
		[
			DetailsView.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE
