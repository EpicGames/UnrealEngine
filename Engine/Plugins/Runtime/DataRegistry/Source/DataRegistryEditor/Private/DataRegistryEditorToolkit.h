// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryTypes.h"
#include "DataTableEditorUtils.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Views/SHeaderRow.h"

class FSpawnTabArgs;
class FToolBarBuilder;
class SSearchBox;
class UDataRegistry;
template <typename ItemType> class SListView;

class ITableRow;
class STableViewBase;

// TODO This is heavily based off of FDataTableEditor, could possibly refactor some of this into a parent class

/** Viewer/editor for a DataRegistry */
class FDataRegistryEditorToolkit : public FAssetEditorToolkit
{
private:
	friend class SDataRegistryListViewRow;
	using Super = FAssetEditorToolkit;
	
public:	

	FDataRegistryEditorToolkit();
	virtual ~FDataRegistryEditorToolkit();

	static TSharedRef<FDataRegistryEditorToolkit> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UDataRegistry* InDataRegistry);

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	virtual void InitDataRegistryEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataRegistry* Registry);

	/**	Spawns the tab with the details view inside */
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/**	Spawns the tab with the data table inside */
	TSharedRef<SDockTab> SpawnTab_RowList(const FSpawnTabArgs& Args);


	/** Get the data table being edited */
	const UDataRegistry* GetDataRegistry() const;

	void HandlePostChange();

	void SetHighlightedRow(FName Name);

	FText GetFilterText() const;

	FSlateColor GetRowTextColor(FName RowName) const;

	const FDataRegistrySourceItemId* GetSourceItemForName(FName DebugName) const;

protected:

	
	void RefreshCachedDataRegistry(const FName InCachedSelection = NAME_None, const bool bUpdateEvenIfValid = false);

	void UpdateVisibleRows(const FName InCachedSelection = NAME_None, const bool bUpdateEvenIfValid = false);

	void RestoreCachedSelection(const FName InCachedSelection, const bool bUpdateEvenIfValid = false);

	void OnFilterTextChanged(const FText& InFilterText);
	void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	virtual void PostRegenerateMenusAndToolbars() override;

	FReply OnFindRowInContentBrowserClicked();
	void OnEditDataTableStructClicked();

	FText GetCellText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const;
	FText GetCellToolTipText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const;

	TSharedRef<SVerticalBox> CreateContentBox();

	float GetRowNameColumnWidth() const;
	void RefreshRowNameColumnWidth();

	float GetRowNumberColumnWidth() const;
	void RefreshRowNumberColumnWidth();

	float GetColumnWidth(const int32 ColumnIndex) const;

	void OnColumnResized(const float NewWidth, const int32 ColumnIndex);

	void OnRowNameColumnResized(const float NewWidth);

	void OnRowNumberColumnResized(const float NewWidth);

	/** Make the widget for a row entry in the data table row list view */
	TSharedRef<ITableRow> MakeRowWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable);

	void OnRowSelectionChanged(FDataTableEditorRowListViewDataPtr InNewSelection, ESelectInfo::Type InSelectInfo);

	void CopySelectedRow();

	/** Helper function for creating and registering the tab containing the row list */
	virtual void CreateAndRegisterRowListTab(const TSharedRef<class FTabManager>& InTabManager);

	/** Helper function for creating and registering the tab containing the properties tab */
	virtual void CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager);

	void OnCopyClicked();
	void OnRefreshClicked();

	void OnDataAcquired(EDataRegistryAcquireStatus AcquireStatus);

	void SetDefaultSort();
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	void OnColumnNumberSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	void OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	void ExtendToolbar(TSharedPtr<FExtender> Extender);
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

private:
	UDataRegistry* GetEditableDataRegistry() const;

protected:

	/** Struct holding information about the current column widths */
	struct FColumnWidth
	{
		FColumnWidth()
			: bIsAutoSized(true)
			, CurrentWidth(0.0f)
		{
		}

		/** True if this column is being auto-sized rather than sized by the user */
		bool bIsAutoSized;
		/** The width of the column, either sized by the user, or auto-sized */
		float CurrentWidth;
	};

	TSharedPtr<SWidget> RowListTabWidget;
	TSharedPtr<SSearchBox> SearchBoxWidget;

	/** Array of the columns that are available for editing */
	TArray<FDataTableEditorColumnHeaderDataPtr> AvailableColumns;

	/** Array of the rows that are available for editing */
	TArray<FDataTableEditorRowListViewDataPtr> AvailableRows;

	/** Map of debug names to source details */
	TMap<FName, FDataRegistrySourceItemId> SourceItemMap;

	/** Array of the rows that match the active filter(s) */
	TArray<FDataTableEditorRowListViewDataPtr> VisibleRows;

	/** Header row containing entries for each column in AvailableColumns */
	TSharedPtr<SHeaderRow> ColumnNamesHeaderRow;

	/** List view responsible for showing the rows in VisibleRows for each entry in AvailableColumns */
	TSharedPtr<SListView<FDataTableEditorRowListViewDataPtr>> CellsListView;

	/** Width of the row name column */
	float RowNameColumnWidth;

	/** Width of the row number column */
	float RowNumberColumnWidth;

	/** If this is set, we asked the registry for info on some ids */
	TArray<FDataRegistrySourceItemId> PendingSourceData;

	/** Widths of data table cell columns */
	TArray<FColumnWidth> ColumnWidths;

	/** The name of the currently selected row */
	FName HighlightedRowName;

	/** The visible row index of the currently selected row */
	int32 HighlightedVisibleRowIndex;

	/** The current filter text applied to the data table */
	FText ActiveFilterText;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Details view */
	TSharedPtr< class IDetailsView > DetailsView;

	/** UI for the "Stack" tab */
	TSharedPtr<SWidget> StackTabWidget;

	static const FName AppIdentifier;
	static const FName RowListTabId;
	static const FName PropertiesTabId;
	static const FName StackTabId;
	static const FName RowNumberColumnId;
	static const FName RowNameColumnId;
	static const FName RowSourceColumnId;
	static const FName RowResolvedColumnId;
	

	
};
