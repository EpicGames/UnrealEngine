// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryFactory.h"

#include "DataRegistry.h"
#include "ClassViewerFilter.h"
#include "Modules/ModuleManager.h"
#include "Kismet2/SClassPickerDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistryFactory)

#define LOCTEXT_NAMESPACE "DataRegistryEditor"

UDataRegistryFactory::UDataRegistryFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDataRegistry::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

// Copy paste of EditorFactories implementation, which is not publicly exposed
class FAssetClassParentFilter : public IClassViewerFilter
{
public:
	FAssetClassParentFilter()
		: DisallowedClassFlags(CLASS_None)
	{}

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		bool bAllowed= !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;

		return bAllowed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

bool UDataRegistryFactory::ConfigureProperties()
{
	// nullptr the DataRegistryClass so we can check for selection
	DataRegistryClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	// This doesn't currently seem to work
	// Options.InitiallySelectedClass = UDataRegistry::StaticClass();

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(UDataRegistry::StaticClass());

	const FText TitleText = LOCTEXT("CreateDataRegistryOptions", "Pick Data Registry Base Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UDataRegistry::StaticClass());

	if (bPressedOk)
	{
		DataRegistryClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UDataRegistryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (DataRegistryClass)
	{
		Class = DataRegistryClass;
	}

	ensure(0 != (RF_Public & Flags));
	UDataRegistry* DataRegistry = NewObject<UDataRegistry>(InParent, Class, Name, Flags | RF_Transactional);
	return DataRegistry;
}

#undef LOCTEXT_NAMESPACE
