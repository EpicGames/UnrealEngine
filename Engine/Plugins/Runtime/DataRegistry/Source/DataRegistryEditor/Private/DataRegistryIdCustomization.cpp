// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryIdCustomization.h"
#include "DataRegistrySubsystem.h"
#include "DataTableEditorUtils.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "PropertyCustomizationHelpers.h"
#include "GameplayTagsEditorModule.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistryIdCustomization)

DEFINE_LOG_CATEGORY_STATIC(LogDataRegistryIdCustomization, Log, All);

#define LOCTEXT_NAMESPACE "DataRegistryEditor"

FText FDataRegistryIdEditWrapper::GetPreviewDescription() const
{
	UDataRegistrySubsystem* Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();

	if (Subsystem)
	{
		return Subsystem->GetDisplayTextForId(RegistryId);
	}

	return RegistryId.ToText();
}


void SDataRegistryItemNameWidget::Construct(const FArguments& InArgs)
{
	OnGetDisplayText = InArgs._OnGetDisplayText;
	OnGetId = InArgs._OnGetId;
	OnSetId = InArgs._OnSetId;
	OnGetCustomItemNames = InArgs._OnGetCustomItemNames;
	bAllowClear = InArgs._bAllowClear;

	Subsystem = GEngine->GetEngineSubsystem<UDataRegistrySubsystem>();
	check(Subsystem);

	FPropertyComboBoxArgs NameArgs(nullptr,
		FOnGetPropertyComboBoxStrings::CreateSP(this, &SDataRegistryItemNameWidget::OnGetNameStrings),
		FOnGetPropertyComboBoxValue::CreateSP(this, &SDataRegistryItemNameWidget::OnGetNameValueString),
		FOnPropertyComboBoxValueSelected::CreateSP(this, &SDataRegistryItemNameWidget::OnNameSelected));
	NameArgs.ShowSearchForItemCount = 1;

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SAssignNew(ValueSwitcher, SWidgetSwitcher)
			.WidgetIndex(0)
			+ SWidgetSwitcher::Slot()
			[	
				PropertyCustomizationHelpers::MakePropertyComboBox(NameArgs)
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &SDataRegistryItemNameWidget::GetTagContent)
				.OnMenuOpenChanged(this, &SDataRegistryItemNameWidget::OnTagUIOpened)
				.MenuPlacement(MenuPlacement_BelowAnchor)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SDataRegistryItemNameWidget::OnGetNameValueText)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(2, 0, 0, 0))
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
			.DesiredSizeOverride(FVector2D(16, 16))
			.ToolTipText(LOCTEXT("InvalidRegistryId", "Unknown Item Name, this will not work at runtime with the current registry settings"))
			.Visibility(this, &SDataRegistryItemNameWidget::GetWarningVisibility)
		]
	];
}


void SDataRegistryItemNameWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FDataRegistryId CurrentId = OnGetId.Execute();

	// Refresh if type has changed, or it's a custom type that might be different based on external factors
	if (CurrentId.RegistryType != CachedIdValue.RegistryType || CurrentId.RegistryType == FDataRegistryType::CustomContextType)
	{
		bool bClearInvalid = CachedIdValue.RegistryType.IsValid();
		CachedIdValue = CurrentId;
		OnTypeChanged(bClearInvalid);
	}
	else if (CurrentId.ItemName != CachedIdValue.ItemName)
	{
		CachedIdValue.ItemName = CurrentId.ItemName;
	}
}

void SDataRegistryItemNameWidget::OnNameSelected(const FString& NameString)
{
	FDataRegistryId NewId = CachedIdValue;
	NewId.ItemName = FName(*NameString);

	OnSetId.Execute(NewId);
}

void SDataRegistryItemNameWidget::OnTypeChanged(bool bClearInvalid)
{
	int32 SwitcherIndex = 0;
	CachedIds.Reset();

	if (CachedIdValue.RegistryType == FDataRegistryType::CustomContextType)
	{
		// For fake custom type, use callback
		TArray<FName> CustomNames;
		if (OnGetCustomItemNames.IsBound())
		{
			OnGetCustomItemNames.Execute(CustomNames);

			for (FName CustomName : CustomNames)
			{
				CachedIds.Add(FDataRegistryId(FDataRegistryType::CustomContextType, CustomName));
			}
		}
	}

	UDataRegistry* Registry = Subsystem->GetRegistryForType(CachedIdValue.RegistryType);

	if (Registry)
	{
		Registry->GetPossibleRegistryIds(CachedIds);

		FDataRegistryIdFormat IdFormat = Registry->GetIdFormat();

		if (IdFormat.BaseGameplayTag.IsValid())
		{
			// Switch to the tag widget
			SwitcherIndex = 1;
			CachedBaseGameplayTag = IdFormat.BaseGameplayTag.ToString();
		}
	}

	if (!CachedIdValue.ItemName.IsNone() && !CachedIds.Contains(CachedIdValue) && bClearInvalid)
	{
		UE_LOGF(LogDataRegistryIdCustomization, Warning, "%ls reset to None because it's not a valid data registry ID", *CachedIdValue.ToString());
		// Name no longer valid, clear
		OnSetId.Execute(FDataRegistryId(CachedIdValue.RegistryType, NAME_None));
	}

	ValueSwitcher->SetActiveWidgetIndex(SwitcherIndex);
}

TSharedRef<SWidget> SDataRegistryItemNameWidget::GetTagContent()
{
	CachedTag = MakeShared<FGameplayTag>();

	FDataRegistryId CurrentId = OnGetId.Execute();
	if (CurrentId.IsValid())
	{
		FGameplayTag NameTag = FGameplayTag::RequestGameplayTag(CurrentId.ItemName, false);

		if (NameTag.IsValid())
		{
			*CachedTag = NameTag;
		}
	}

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(400)
		[
			IGameplayTagsEditorModule::Get().MakeGameplayTagWidget(FOnSetGameplayTag::CreateSP(this, &SDataRegistryItemNameWidget::OnTagChanged), CachedTag, CachedBaseGameplayTag)
		];
}

void SDataRegistryItemNameWidget::OnTagUIOpened(bool bIsOpened)
{
	if (bIsOpened)
	{
		// TODO Investigate tag focus issues in general
	/*	TSharedPtr<SGameplayTagWidget> TagWidget = LastTagWidget.Pin();
		if (TagWidget.IsValid())
		{
			EditButton->SetMenuContentWidgetToFocus(TagWidget->GetWidgetToFocusOnOpen());
		}*/
	}
}

void SDataRegistryItemNameWidget::OnTagChanged(const FGameplayTag& NewTag)
{
	OnNameSelected(NewTag.GetTagName().ToString());
}

void SDataRegistryItemNameWidget::OnGetNameStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	// Always add none
	OutStrings.Add(MakeShared<FString>(TEXT("None")));
	OutRestrictedItems.Add(false);

	for (const FDataRegistryId& CurrentID : CachedIds)
	{
		OutStrings.Add(MakeShared<FString>(CurrentID.ItemName.ToString()));
		OutRestrictedItems.Add(false);
	}
}

FString SDataRegistryItemNameWidget::OnGetNameValueString() const
{
	return OnGetNameValueText().ToString();
}

FText SDataRegistryItemNameWidget::OnGetNameValueText() const
{
	return OnGetDisplayText.Execute();
}

EVisibility SDataRegistryItemNameWidget::GetWarningVisibility() const
{
	if (CachedIdValue.RegistryType.IsValid() && !CachedIdValue.ItemName.IsNone() && !CachedIds.Contains(CachedIdValue))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

namespace DataRegistryCustomizationPrivate
{
	template <typename T>
	T* GetPropertyData(TSharedPtr<IPropertyHandle> StructPropertyHandle)
	{
		if (StructPropertyHandle.IsValid())
		{
			void* StructAddress = nullptr;
			if (StructPropertyHandle->GetValueData(StructAddress) == FPropertyAccess::Success)
			{
				return static_cast<T*>(StructAddress);
			}
		}

		return nullptr;
	}
}

TSharedRef<IPropertyTypeCustomization> FDataRegistryIdCustomization::MakeInstance()
{
	return MakeShareable(new FDataRegistryIdCustomization);
}

void FDataRegistryIdCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;
	
	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	];

	FDataTableEditorUtils::AddSearchForReferencesContextMenu(HeaderRow, FExecuteAction::CreateSP(this, &FDataRegistryIdCustomization::OnSearchForReferences));
}

void FDataRegistryIdCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	/** Get all the existing property handles, need the name inside the type */
	TypePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryId, RegistryType));
	NamePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryId, ItemName));

	if (TypePropertyHandle->IsValidHandle() && NamePropertyHandle->IsValidHandle())
	{
		StructBuilder.AddProperty(TypePropertyHandle.ToSharedRef());

		/** Construct a combo box widget to select name */
		StructBuilder.AddCustomRow(LOCTEXT("ItemName", "Item Name"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ItemName", "Item Name"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			.ValueContent()
			.MaxDesiredWidth(0.0f)
			[
				SNew(SDataRegistryItemNameWidget)
				.OnGetDisplayText(this, &FDataRegistryIdCustomization::OnGetNameValueText)
				.OnGetId(this, &FDataRegistryIdCustomization::GetCurrentValue)
				.OnSetId(this, &FDataRegistryIdCustomization::SetCurrentValue)
				.bAllowClear(true)
			];
	}
}

FDataRegistryId FDataRegistryIdCustomization::GetCurrentValue() const
{
	FDataRegistryId OutId;
	const FDataRegistryType* DataRegistryType = DataRegistryCustomizationPrivate::GetPropertyData<FDataRegistryType>(TypePropertyHandle);
	if (DataRegistryType && NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
	{
		OutId.RegistryType = *DataRegistryType;
		NamePropertyHandle->GetValue(OutId.ItemName);
	}

	return OutId;
}

void FDataRegistryIdCustomization::SetCurrentValue(FDataRegistryId NewId)
{
	const FDataRegistryId CurrentId = GetCurrentValue();

	// Always set value, only set type if different
	NamePropertyHandle->SetValue(NewId.ItemName);

	if (NewId.RegistryType != CurrentId.RegistryType)
	{
		if (FDataRegistryType* DataRegistryType = DataRegistryCustomizationPrivate::GetPropertyData<FDataRegistryType>(TypePropertyHandle))
		{
			*DataRegistryType = NewId.RegistryType;
		}
	}
}

void FDataRegistryIdCustomization::OnSearchForReferences()
{
	// TODO implement or remove
}

FString FDataRegistryIdCustomization::OnGetTypeValueString() const
{
	FDataRegistryId CurrentId = GetCurrentValue();
	if (CurrentId.RegistryType.IsValid())
	{
		return CurrentId.RegistryType.ToString();
	}

	FDataRegistryType TempRegistry;
	if (TypePropertyHandle.IsValid() && TypePropertyHandle->GetValue(TempRegistry) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}

	return LOCTEXT("NoneValue", "None").ToString();
}

FText FDataRegistryIdCustomization::OnGetNameValueText() const
{
	FDataRegistryId CurrentId = GetCurrentValue();
	if (!CurrentId.ItemName.IsNone())
	{
		return FText::FromName(CurrentId.ItemName);
	}

	FName TempName;
	if (NamePropertyHandle.IsValid() && NamePropertyHandle->GetValue(TempName) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return LOCTEXT("NoneValue", "None");
}

#undef LOCTEXT_NAMESPACE

