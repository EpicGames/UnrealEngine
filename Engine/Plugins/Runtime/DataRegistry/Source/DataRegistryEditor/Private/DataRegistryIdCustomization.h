// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryEditorModule.h"
#include "DataRegistryId.h"
#include "IPropertyTypeCustomization.h"
#include "KismetPins/SGraphPinStructInstance.h"
#include "DataRegistryIdCustomization.generated.h"

struct FGameplayTag;

/** This allows using this customization with blueprint pins */
USTRUCT()
struct FDataRegistryIdEditWrapper : public FPinStructEditWrapper
{
	GENERATED_BODY()

	/** Actual id to edit */
	UPROPERTY(EditAnywhere, Category = Registry, Meta = (ShowOnlyInnerProperties))
	FDataRegistryId RegistryId;

	/** Returns a text representation of the data */
	virtual FText GetPreviewDescription() const override;

	/** Returns what script struct to use to parse the nested data */
	virtual const UScriptStruct* GetDataScriptStruct() const override { return FDataRegistryId::StaticStruct(); }

	/** Returns address of nested data */
	virtual uint8* GetDataMemory() override { return (uint8*)&RegistryId; }
};

/** Raw widget, used by both customization and external callers */
class SDataRegistryItemNameWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataRegistryItemNameWidget)
		: _bAllowClear(true)
	{}		
		SLATE_EVENT(FOnGetDataRegistryDisplayText, OnGetDisplayText)
		SLATE_EVENT(FOnGetDataRegistryId, OnGetId)
		SLATE_EVENT(FOnSetDataRegistryId, OnSetId)
		SLATE_EVENT(FOnGetCustomDataRegistryItemNames, OnGetCustomItemNames)
		SLATE_ARGUMENT(bool, bAllowClear)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

private:
	/** Updates cached values */
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Delegate to refresh the drop down when the selected type changes */
	void OnTypeChanged(bool bClearInvalid);

	/** Change value and forward to delegates */
	void OnNameSelected(const FString& NameString);

	/** Display tag UI */
	TSharedRef<SWidget> GetTagContent();
	void OnTagUIOpened(bool bIsOpened);

	/** Delegate when tag changes */
	void OnTagChanged(const FGameplayTag& NewTag);

	/** Get list of all name */
	void OnGetNameStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;

	/** Slate accessors */
	FString OnGetNameValueString() const;
	FText OnGetNameValueText() const;
	EVisibility GetWarningVisibility() const;

	/** This will never be bad while engine is running */
	class UDataRegistrySubsystem* Subsystem = nullptr;

	/** Cached Id, if this changes then refresh items */
	FDataRegistryId CachedIdValue;

	/** Cached list of ids for current type */
	TArray<FDataRegistryId> CachedIds;

	/** Tag representation of name */
	TSharedPtr<FGameplayTag> CachedTag;

	/** Root gameplay tag to show */
	FString CachedBaseGameplayTag;

	/** Widget switcher for value editor */
	TSharedPtr<class SWidgetSwitcher> ValueSwitcher;

	/** Creation arguments */
	FOnGetDataRegistryDisplayText OnGetDisplayText;
	FOnGetDataRegistryId OnGetId;
	FOnSetDataRegistryId OnSetId;
	FOnGetCustomDataRegistryItemNames OnGetCustomItemNames;
	bool bAllowClear;
};


/**
 *
 */
class FDataRegistryIdCustomization : public IPropertyTypeCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	
	/** Open reference viewer */
	void OnSearchForReferences();

	/** Slate accessors */
	FString OnGetTypeValueString() const;
	FText OnGetNameValueText() const;
	FDataRegistryId GetCurrentValue() const;
	void SetCurrentValue(FDataRegistryId NewId);

	/** Handle to the struct properties being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> TypePropertyHandle;
	TSharedPtr<IPropertyHandle> NamePropertyHandle;

	/** This will never be bad while engine is running */
	class UDataRegistrySubsystem* Subsystem = nullptr;

	/** Cached list of ids for current type */
	TArray<FDataRegistryId> CachedIds;

	/** Tag representation of name */
	TSharedPtr<FGameplayTag> CachedTag;

	/** Widget switcher for value editor */
	TSharedPtr<class SWidgetSwitcher> ValueSwitcher;

};
