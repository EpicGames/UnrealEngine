// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTypeCustomization.h"
#include "DataRegistry.h"
#include "DataRegistrySubsystem.h"
#include "DataRegistryEditorModule.h"
#include "EdGraph/EdGraphSchema.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "DataRegistryEditor"

void FDataRegistryTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	if (StructPropertyHandle.IsValid())
	{
		FName FilterStructName;
		const bool bAllowClear = !(StructPropertyHandle->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);

		if (StructPropertyHandle->HasMetaData(FDataRegistryType::ItemStructMetaData))
		{
			const FString& RowType = StructPropertyHandle->GetMetaData(FDataRegistryType::ItemStructMetaData);
			FilterStructName = FName(*RowType);
		}

		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(0.0f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(1, 0)
			[
				PropertyCustomizationHelpers::MakePropertyComboBox(InStructPropertyHandle, FOnGetPropertyComboBoxStrings::CreateStatic(&FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings, bAllowClear, FilterStructName))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(1, 0)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(22.0f)
				.HeightOverride(22.0f)
				.ToolTipText(this, &FDataRegistryTypeCustomization::GetOpenAssetTooltip)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &FDataRegistryTypeCustomization::OnClickOpenAsset)
					.ContentPadding(0.0f)
					.IsFocusable(false)
					.Visibility(this, &FDataRegistryTypeCustomization::GetOpenAssetVisibility)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("SystemWideCommands.SummonOpenAssetDialog"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		];
	}
}

const FDataRegistryType* FDataRegistryTypeCustomization::GetPropertyValue() const
{
	void* PropertyData;
	if (StructPropertyHandle && StructPropertyHandle->GetValueData(PropertyData) == FPropertyAccess::Success)
	{
		return reinterpret_cast<FDataRegistryType*>(PropertyData);
	}

	return nullptr;
}

UDataRegistry* FDataRegistryTypeCustomization::GetDataRegistry() const
{
	if (const UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get())
	{
		if (const FDataRegistryType* RegistryType = GetPropertyValue())
		{
			return DataRegistrySubsystem->GetRegistryForType(*RegistryType);
		}
	}

	return nullptr;
}

FReply FDataRegistryTypeCustomization::OnClickOpenAsset()
{
	if (UDataRegistry* DataRegistry = GetDataRegistry())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DataRegistry);
	}
	return FReply::Handled();
}

EVisibility FDataRegistryTypeCustomization::GetOpenAssetVisibility() const
{
	return (GetDataRegistry() != nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FDataRegistryTypeCustomization::GetOpenAssetTooltip() const
{
	if (const UDataRegistry* DataRegistry = GetDataRegistry())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Asset"), FText::AsCultureInvariant(DataRegistry->GetName()));
		return FText::Format(LOCTEXT("OpenSpecificDataRegistry", "Open '{Asset}' in the editor"), Args);
	}

	return LOCTEXT("OpenDataRegistry", "Open the Data Registry in the editor");
}

void SDataRegistryTypeGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SDataRegistryTypeGraphPin::GetDefaultValueWidget()
{
	FString DefaultString = GraphPinObj->GetDefaultAsString();
	CurrentType = FDataRegistryType(*DefaultString);

	return SNew(SVerticalBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FDataRegistryEditorModule::MakeDataRegistryTypeSelector(
				FOnGetDataRegistryDisplayText::CreateSP(this, &SDataRegistryTypeGraphPin::GetDisplayText),
				FOnSetDataRegistryType::CreateSP(this, &SDataRegistryTypeGraphPin::OnTypeSelected),
				true)
		];
}

void SDataRegistryTypeGraphPin::OnTypeSelected(FDataRegistryType AssetType)
{
	CurrentType = AssetType;
	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, CurrentType.ToString());
}

FText SDataRegistryTypeGraphPin::GetDisplayText() const
{
	return FText::AsCultureInvariant(CurrentType.ToString());
}

#undef LOCTEXT_NAMESPACE

