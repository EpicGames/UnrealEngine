// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryId.h"
#include "IPropertyTypeCustomization.h"
#include "SGraphPin.h"

class UDataRegistry;
struct FDataRegistryType;

/** Data Registry type, reads list from subsystem */
class FDataRegistryTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FDataRegistryTypeCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	/** Handle to the struct property being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Returns the data of the underlying property */
	const FDataRegistryType* GetPropertyValue() const;

	/** Returns the data registry associated with this registry type, if there is one */
	UDataRegistry* GetDataRegistry() const;

	/** Creates the string list */
	void GenerateComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<class SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems);

	/** Functionality for the "open asset" button */
	FReply OnClickOpenAsset();
	EVisibility GetOpenAssetVisibility() const;
	FText GetOpenAssetTooltip() const;

};

/** Graph pin version of UI */
class SDataRegistryTypeGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDataRegistryTypeGraphPin) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:

	void OnTypeSelected(FDataRegistryType AssetType);
	FText GetDisplayText() const;

	FDataRegistryType CurrentType;
};
