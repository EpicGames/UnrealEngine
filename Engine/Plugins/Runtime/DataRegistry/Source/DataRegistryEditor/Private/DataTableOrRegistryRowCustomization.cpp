// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableOrRegistryRowCustomization.h"
#include "Components/WrapBox.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "DataRegistryEditorModule.h"
#include "IDetailChildrenBuilder.h"
#include "DataRegistryIdCustomization.h"
#include "DataRegistrySubsystem.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "DataRegistryOrTableRow"


// Makes a new instance of this detail layout class for a specific detail view requesting it
TSharedRef<IPropertyTypeCustomization> FDataRegistryOrTableRowCustomization::MakeInstance()
{
	return MakeShareable(new FDataRegistryOrTableRowCustomization);
}

void FDataRegistryOrTableRowCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;

	FilterStructName = NAME_None;
	RegistryFilterStructName = NAME_None;

	if (StructPropertyHandle->HasMetaData(FDataRegistryType::ItemStructMetaData))
	{
		const FString& RowType = StructPropertyHandle->GetMetaData(FDataRegistryType::ItemStructMetaData);
		FilterStructName = FName(*RowType);

		// The registry filter needs the short name, but the data table is very slow unless the long name is used, so just extract the suffix
		int32 ExtIndex = -1;
		if (RowType.FindLastChar('.', ExtIndex))
		{
			RegistryFilterStructName = FName(RowType.Mid(ExtIndex+1));
		}
		else
		{
			RegistryFilterStructName = FilterStructName;
		}		
	}
}

void FDataRegistryOrTableRowCustomization::GetDataSourceComboStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	OutComboBoxStrings.Add(MakeShared<FString>(LOCTEXT("DataTable", "Data Table").ToString()));
	OutComboBoxStrings.Add(MakeShared<FString>(LOCTEXT("DataRegistry", "Data Registry").ToString()));
	OutRestrictedItems.Add(false);
	OutRestrictedItems.Add(false);	
}

FString FDataRegistryOrTableRowCustomization::GetDataSourceTypeValueString() const
{
	if (!UseRegistryPropertyHandle.IsValid() || !UseRegistryPropertyHandle->IsValidHandle())
	{
		return LOCTEXT("NoneValue", "None").ToString();	
	}

	bool bUseDataRegistryId = false;
	const FPropertyAccess::Result SourceResult = UseRegistryPropertyHandle->GetValue(bUseDataRegistryId);
	if (SourceResult == FPropertyAccess::Success)
	{
		if (bUseDataRegistryId)
		{
			return LOCTEXT("DataRegistry", "Data Registry").ToString();
		}
		else
		{
			return LOCTEXT("DataTable", "Data Table").ToString();	
		}
	}
	else if (SourceResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("DataTable_None", "None").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}


void FDataRegistryOrTableRowCustomization::OnDataSourceSelected(const FString& String)
{
	const bool bUsingDataRegistry = String == LOCTEXT("DataRegistry", "Data Registry").ToString();

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (CheckHandle(UseRegistryPropertyHandle))
	{
		UseRegistryPropertyHandle->SetValue(bUsingDataRegistry);
	}

	if (bUsingDataRegistry)
	{
		// clear the data table values to prevent hidden references to a resource
		if (CheckHandle(TablePropertyHandle) && CheckHandle(TableRowNamePropertyHandle))
		{
			TablePropertyHandle->SetValue(static_cast<UDataTable*>(nullptr));
			TableRowNamePropertyHandle->SetValue(NAME_None);
		}
	}
	else
	{
		// clear the data registry values to prevent hidden references to the registry
		if (CheckHandle(RegistryTypeNamePropertyHandle) && CheckHandle(RegistryItemNamePropertyHandle))
		{
			RegistryTypeNamePropertyHandle->SetValue(FDataRegistryType().GetName());
			RegistryItemNamePropertyHandle->SetValue(NAME_None);
		}
	}
}

FDataRegistryId FDataRegistryOrTableRowCustomization::GetCurrentDataRegistryValue() const
{
	FDataRegistryId Result {};

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (CheckHandle(RegistryTypeNamePropertyHandle) && CheckHandle(RegistryItemNamePropertyHandle))
	{
		RegistryTypeNamePropertyHandle->GetValue(static_cast<FName&>(Result.RegistryType));
		RegistryItemNamePropertyHandle->GetValue(Result.ItemName);		
	}

	return Result;

}

void FDataRegistryOrTableRowCustomization::SetCurrentDataRegistryValue(FDataRegistryId NewValue)
{
	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (CheckHandle(RegistryTypeNamePropertyHandle) && CheckHandle(RegistryItemNamePropertyHandle))
	{
		RegistryTypeNamePropertyHandle->SetValue(NewValue.RegistryType.GetName());
		RegistryItemNamePropertyHandle->SetValue(NewValue.ItemName);		
	}
}

FDataRegistryOrTableRow FDataRegistryOrTableRowCustomization::GetCurrentValue() const
{
	FDataRegistryOrTableRow Result;

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (!CheckHandle(UseRegistryPropertyHandle) ||
		!CheckHandle(RegistryTypeNamePropertyHandle) ||
		!CheckHandle(RegistryItemNamePropertyHandle) ||
		!CheckHandle(TablePropertyHandle) ||
		!CheckHandle(TableRowNamePropertyHandle))
	{
		return Result;
	}

	UseRegistryPropertyHandle->GetValue(Result.bUseDataRegistryId);

	if (Result.bUseDataRegistryId)
	{
		RegistryTypeNamePropertyHandle->GetValue(static_cast<FName&>(Result.DataRegistryId.RegistryType));
		RegistryItemNamePropertyHandle->GetValue(Result.DataRegistryId.ItemName);
	}
	else
	{
		UObject* SourceDataTable = nullptr;

		TablePropertyHandle->GetValue(SourceDataTable);
		Result.DataTableRow.DataTable = Cast<UDataTable>(SourceDataTable);

		TableRowNamePropertyHandle->GetValue(Result.DataTableRow.RowName);
	}

	return Result;
}

FString FDataRegistryOrTableRowCustomization::OnGetDataRegistryTypeValueString() const
{
	FDataRegistryOrTableRow CurrentId = GetCurrentValue();
	if (CurrentId.bUseDataRegistryId)
	{
		return CurrentId.DataRegistryId.RegistryType.ToString();
	}

	return LOCTEXT("NoneValue", "None").ToString();
}

bool FDataRegistryOrTableRowCustomization::ShouldFilterDataTableAsset(const struct FAssetData& AssetData)
{
	if (!FilterStructName.IsNone())
	{
		static const FName RowStructureTagName("RowStructure");
		UScriptStruct* RowFilterStruct = UClass::TryFindTypeSlow<UScriptStruct>(FilterStructName.ToString());
		FString RowStructure;
		if (AssetData.GetTagValue<FString>(RowStructureTagName, RowStructure))
		{
			// This is slow, but at the moment we don't have an alternative to the short struct name search
			UScriptStruct* RowStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowStructure);

			if (RowStruct && RowFilterStruct && RowStruct->IsChildOf(RowFilterStruct))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void FDataRegistryOrTableRowCustomization::OnGetDataTableRowStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	FDataRegistryOrTableRow CurrentId = GetCurrentValue();
	if (CurrentId.bUseDataRegistryId)
	{
		return;
	}

	TArray<FName> AllRowNames;
	if (CurrentId.DataTableRow.DataTable != nullptr)
	{
		for (TMap<FName, uint8*>::TConstIterator Iterator(CurrentId.DataTableRow.DataTable->GetRowMap()); Iterator; ++Iterator)
		{
			AllRowNames.Add(Iterator.Key());
		}

		// Sort the names alphabetically.
		AllRowNames.Sort(FNameLexicalLess());
	}

	for (const FName& RowName : AllRowNames)
	{
		OutStrings.Add(MakeShared<FString>(RowName.ToString()));
		OutRestrictedItems.Add(false);
	}
}

FString FDataRegistryOrTableRowCustomization::OnGetDataTableRowValueString() const
{
	if (!TableRowNamePropertyHandle.IsValid() || !TableRowNamePropertyHandle->IsValidHandle())
	{
		return FString();
	}

	FName RowNameValue;
	const FPropertyAccess::Result RowResult = TableRowNamePropertyHandle->GetValue(RowNameValue);
	if (RowResult == FPropertyAccess::Success)
	{
		if (RowNameValue.IsNone())
		{
			return LOCTEXT("DataTable_None", "None").ToString();
		}
		return RowNameValue.ToString();
	}
	else if (RowResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("DataTable_None", "None").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}

FText FDataRegistryOrTableRowCustomization::OnGetDataRegistryNameValueText() const
{
	FDataRegistryId CurrentId = GetCurrentDataRegistryValue();
	if (!CurrentId.ItemName.IsNone())
	{
		return FText::FromName(CurrentId.ItemName);
	}

	FName TempName;
	if (RegistryItemNamePropertyHandle.IsValid() && RegistryItemNamePropertyHandle->GetValue(TempName) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return LOCTEXT("NoneValue", "None");
}

EVisibility FDataRegistryOrTableRowCustomization::GetOpenDataRegistryAssetVisibility() const
{
	FDataRegistryId CurrentId = GetCurrentDataRegistryValue();
	return CurrentId.RegistryType.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FDataRegistryOrTableRowCustomization::OnClickOpenDataRegistryAsset()
{
	FDataRegistryId CurrentId = GetCurrentDataRegistryValue();

	if (const UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get())
	{
		if (UDataRegistry* DataRegistry = DataRegistrySubsystem->GetRegistryForType(CurrentId.RegistryType))
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DataRegistry);
		}
	}

	return FReply::Handled();
}


FText FDataRegistryOrTableRowCustomization::GetOpenDataRegistryAssetTooltip() const
{
	FDataRegistryId CurrentId = GetCurrentDataRegistryValue();

	if (const UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get())
	{
		if (UDataRegistry* DataRegistry = DataRegistrySubsystem->GetRegistryForType(CurrentId.RegistryType))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Asset"), FText::AsCultureInvariant(DataRegistry->GetName()));
			return FText::Format(LOCTEXT("OpenSpecificDataRegistry", "Open '{Asset}' in the editor"), Args);
		}
	}

	return LOCTEXT("OpenDataRegistry", "Open the Data Registry in the editor");
}


void FDataRegistryOrTableRowCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	// get the child property handles
	UseRegistryPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryOrTableRow, bUseDataRegistryId));
	if (!CheckHandle(UseRegistryPropertyHandle))
	{
		return;
	}

	TSharedPtr<IPropertyHandle> RegistryIdPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryOrTableRow, DataRegistryId));

	if (!CheckHandle(RegistryIdPropertyHandle))
	{
		return;
	}

	TSharedPtr<IPropertyHandle> RegistryTypePropertyHandle = RegistryIdPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryId, RegistryType));
	if (!CheckHandle(RegistryTypePropertyHandle))
	{
		return;
	}

	RegistryTypeNamePropertyHandle = RegistryTypePropertyHandle->GetChildHandle(FName("Name"));
	if (!CheckHandle(RegistryTypeNamePropertyHandle))
	{
		return;
	}

	RegistryItemNamePropertyHandle = RegistryIdPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryId, ItemName));
	if (!CheckHandle(RegistryItemNamePropertyHandle))
	{
		return;
	}

	TSharedPtr<IPropertyHandle> DataTableRowPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataRegistryOrTableRow, DataTableRow));
	if (!CheckHandle(DataTableRowPropertyHandle))
	{
		return;
	}

	TablePropertyHandle = DataTableRowPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataTableRowHandle, DataTable));
	if (!CheckHandle(TablePropertyHandle))
	{
		return;
	}

	TableRowNamePropertyHandle = DataTableRowPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataTableRowHandle, RowName));
	if (!CheckHandle(TableRowNamePropertyHandle))
	{
		return;
	}

	FPropertyComboBoxArgs SelectSourceTypeArgs(UseRegistryPropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateSP(this, &FDataRegistryOrTableRowCustomization::GetDataSourceComboStrings),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FDataRegistryOrTableRowCustomization::GetDataSourceTypeValueString),
		FOnPropertyComboBoxValueSelected::CreateSP(this, &FDataRegistryOrTableRowCustomization::OnDataSourceSelected)
	);

	FPropertyComboBoxArgs DataRegistryIdTypeArgs(RegistryTypeNamePropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateStatic(FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings, true, RegistryFilterStructName),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FDataRegistryOrTableRowCustomization::OnGetDataRegistryTypeValueString));
	DataRegistryIdTypeArgs.ShowSearchForItemCount = 1;

	FPropertyComboBoxArgs DataTableRowStringsArgs(TableRowNamePropertyHandle, 
		FOnGetPropertyComboBoxStrings::CreateSP(this, &FDataRegistryOrTableRowCustomization::OnGetDataTableRowStrings), 
		FOnGetPropertyComboBoxValue::CreateSP(this, &FDataRegistryOrTableRowCustomization::OnGetDataTableRowValueString));
	DataTableRowStringsArgs.ShowSearchForItemCount = 1;

	auto IsUsingDataRegistry = [PropertyHandleWP = UseRegistryPropertyHandle.ToWeakPtr()]()
	{
		bool bResult = false;

		TSharedPtr<IPropertyHandle> UseRegistryPropertyHandle = PropertyHandleWP.Pin();

		if (UseRegistryPropertyHandle.IsValid() && UseRegistryPropertyHandle->IsValidHandle())
		{
			UseRegistryPropertyHandle->GetValue(bResult);
		}
		return bResult ? EVisibility::Visible : EVisibility::Collapsed;
	};

	auto IsUsingDataTable = [PropertyHandleWP = UseRegistryPropertyHandle.ToWeakPtr()]()
	{
		bool bResult = false;

		TSharedPtr<IPropertyHandle> UseRegistryPropertyHandle = PropertyHandleWP.Pin();

		if (UseRegistryPropertyHandle.IsValid() && UseRegistryPropertyHandle->IsValidHandle())
		{
			UseRegistryPropertyHandle->GetValue(bResult);
		}
		return bResult ? EVisibility::Collapsed : EVisibility::Visible;
	};

	auto IsUsingDataTableAndSourceSelected = [TablePropertyHandleWP = TablePropertyHandle.ToWeakPtr(), IsUsingDataTable]() {

		if (IsUsingDataTable() == EVisibility::Collapsed) 
		{
			return EVisibility::Collapsed;
		}

		UObject* SourceDataTable = nullptr;

		TSharedPtr<IPropertyHandle> TablePropertyHandle = TablePropertyHandleWP.Pin();

		if (TablePropertyHandle.IsValid() && TablePropertyHandle->IsValidHandle())
		{
			TablePropertyHandle->GetValue(SourceDataTable);
		}

		return SourceDataTable != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
	};

	auto IsUsingDataRegistryAndSourceSelected = [RegistryTypeNamePropertyHandleWP = RegistryTypeNamePropertyHandle.ToWeakPtr(), IsUsingDataRegistry]() {

		if (IsUsingDataRegistry() == EVisibility::Collapsed) 
		{
			return EVisibility::Collapsed;
		}

		FName RegistryType = NAME_None;

		TSharedPtr<IPropertyHandle> RegistryTypeNamePropertyHandle = RegistryTypeNamePropertyHandleWP.Pin();

		if (RegistryTypeNamePropertyHandle.IsValid() && RegistryTypeNamePropertyHandle->IsValidHandle())
		{
			RegistryTypeNamePropertyHandle->GetValue(RegistryType);
		}

		return RegistryType.IsNone() ? EVisibility::Collapsed : EVisibility::Visible;
	};

	FUIAction CopyAction;
	FUIAction PasteAction;
	InStructPropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

	StructBuilder.AddCustomRow(LOCTEXT("DataTableOrRegistry", "Data Table Or Registry"))
		.CopyAction(CopyAction)
		.PasteAction(PasteAction)
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(800.f)
		[
			SNew(SWrapBox)
			.UseAllottedSize(true)
			.Orientation(Orient_Horizontal)
			.HAlign(HAlign_Fill)			
			+ SWrapBox::Slot()
				.FillEmptySpace(false)
				[
					PropertyCustomizationHelpers::MakePropertyComboBox(SelectSourceTypeArgs)
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SObjectPropertyEntryBox)
					.PropertyHandle(TablePropertyHandle)
					.AllowedClass(UDataTable::StaticClass())
					.OnShouldFilterAsset(this, &FDataRegistryOrTableRowCustomization::ShouldFilterDataTableAsset)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataTable))
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SHorizontalBox)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataRegistry))
					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							PropertyCustomizationHelpers::MakePropertyComboBox(DataRegistryIdTypeArgs)
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(1, 0)
						[
							SNew(SBox)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.WidthOverride(22.0f)
								.HeightOverride(22.0f)
								.ToolTipText(this, &FDataRegistryOrTableRowCustomization::GetOpenDataRegistryAssetTooltip)
								[
									SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "SimpleButton")
										.OnClicked(this, &FDataRegistryOrTableRowCustomization::OnClickOpenDataRegistryAsset)
										.ContentPadding(0.0f)
										.IsFocusable(false)
										.Visibility(this, &FDataRegistryOrTableRowCustomization::GetOpenDataRegistryAssetVisibility)
										[
											SNew(SImage)
												.Image(FAppStyle::GetBrush("SystemWideCommands.SummonOpenAssetDialog"))
												.ColorAndOpacity(FSlateColor::UseForeground())
										]
								]
						]
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SBox)
					.Padding(0.0f)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataTableAndSourceSelected))
					.VAlign(VAlign_Center)
					[
						PropertyCustomizationHelpers::MakePropertyComboBox(DataTableRowStringsArgs)
					]
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SDataRegistryItemNameWidget)
					.OnGetDisplayText(this, &FDataRegistryOrTableRowCustomization::OnGetDataRegistryNameValueText)
					.OnGetId(this, &FDataRegistryOrTableRowCustomization::GetCurrentDataRegistryValue)
					.OnSetId(this, &FDataRegistryOrTableRowCustomization::SetCurrentDataRegistryValue)
					.bAllowClear(true)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataRegistryAndSourceSelected))
				]
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Create(
			TAttribute<bool>::CreateLambda([InStructPropertyHandle]() -> bool
			{
				return InStructPropertyHandle->IsValidHandle() && InStructPropertyHandle->DiffersFromDefault();
			}),
			FSimpleDelegate::CreateLambda([InStructPropertyHandle]()
			{
				if (InStructPropertyHandle->IsValidHandle())
				{
					InStructPropertyHandle->ResetToDefault();
				}
			})
		));
}

#undef LOCTEXT_NAMESPACE
