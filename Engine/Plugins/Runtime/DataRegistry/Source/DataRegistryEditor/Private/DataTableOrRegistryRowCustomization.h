// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoftDataRegistryOrTable.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SToolTip.h"

/**
 * Customizes a FDataRegistryOrTableRow to support filtering by item struct, and compacting it to one row 
 */
class FDataRegistryOrTableRowCustomization : public IPropertyTypeCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void GetDataSourceComboStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;
	FString GetDataSourceTypeValueString() const;
	void OnDataSourceSelected(const FString& String);

	FString OnGetDataRegistryTypeValueString() const;
	bool ShouldFilterDataTableAsset(const struct FAssetData& AssetData);

	void OnGetDataTableRowStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;
	FString OnGetDataTableRowValueString() const;

	FDataRegistryOrTableRow GetCurrentValue() const;
	EVisibility GetOpenDataRegistryAssetVisibility() const;
	FReply OnClickOpenDataRegistryAsset();

	FText GetOpenDataRegistryAssetTooltip() const;
	FText OnGetDataRegistryNameValueText() const;
	FDataRegistryId GetCurrentDataRegistryValue() const;
	void SetCurrentDataRegistryValue(FDataRegistryId NewValue);

	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	FName FilterStructName;
	FName RegistryFilterStructName;

	TSharedPtr<IPropertyHandle> UseRegistryPropertyHandle;
	TSharedPtr<IPropertyHandle> RegistryTypeNamePropertyHandle;
	TSharedPtr<IPropertyHandle> RegistryItemNamePropertyHandle;
	TSharedPtr<IPropertyHandle> TablePropertyHandle;
	TSharedPtr<IPropertyHandle> TableRowNamePropertyHandle;
};
