// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataRegistryListViewRow.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "DataRegistryEditorToolkit.h"
#include "DataRegistrySource.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Views/SListView.h"
#include "Layout/WidgetPath.h"

#define LOCTEXT_NAMESPACE "SDataRegistryListViewRowName"

void SDataRegistryListViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	RowDataPtr = InArgs._RowDataPtr;
	CurrentName = MakeShareable(new FName(RowDataPtr->RowId));
	DataRegistryEditor = InArgs._DataRegistryEditor;
	SMultiColumnTableRow<FDataTableEditorRowListViewDataPtr>::Construct(
		FSuperRowType::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
		InOwnerTableView
	);

	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SDataRegistryListViewRow::GetBorder));
}

FReply SDataRegistryListViewRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && RowDataPtr.IsValid() && FEditorDelegates::OnOpenReferenceViewer.IsBound() && DataRegistryEditor.IsValid())
	{
		DataRegistryEditor.Pin().Get()->SetHighlightedRow(RowDataPtr->RowId);

		TSharedRef<SWidget> MenuWidget = MakeRowActionsMenu();

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuWidget, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);
		return FReply::Handled();
	}

	return STableRow::OnMouseButtonUp(MyGeometry, MouseEvent);
}

const FDataRegistrySourceItemId* SDataRegistryListViewRow::FindSource() const
{
	if (DataRegistryEditor.IsValid() && RowDataPtr.IsValid())
	{
		if (const FDataRegistryEditorToolkit* DataRegistryEditorPtr = DataRegistryEditor.Pin().Get())
		{
			return DataRegistryEditorPtr->GetSourceItemForName(RowDataPtr->RowId);
		}
	}

	return nullptr;
}

void SDataRegistryListViewRow::OnSearchForReferences()
{
	if (const FDataRegistrySourceItemId* FoundSource = FindSource())
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Add(FAssetIdentifier(FDataRegistryId::StaticStruct(), FName(FoundSource->ItemId.ToString())));

		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}
}

void SDataRegistryListViewRow::OnOpenSourceAsset()
{
	const FDataRegistrySourceItemId* FoundSource = FindSource();
	const UDataRegistrySource* SourceObject = FoundSource ? FoundSource->CachedSource.Get() : nullptr;
	if (SourceObject)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SourceObject->GetSourceAssetPath());
	}
}

TSharedRef<SWidget> SDataRegistryListViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FDataRegistryEditorToolkit> DataRegistryEditorPtr = DataRegistryEditor.Pin();
	return (DataRegistryEditorPtr.IsValid())
		? MakeCellWidget(IndexInList, ColumnName)
		: SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDataRegistryListViewRow::MakeCellWidget(const int32 InRowIndex, const FName& InColumnId)
{
	int32 ColumnIndex = 0;

	FDataRegistryEditorToolkit* DataRegistryEdit = DataRegistryEditor.Pin().Get();
	TArray<FDataTableEditorColumnHeaderDataPtr>& AvailableColumns = DataRegistryEdit->AvailableColumns;

	if (InColumnId.IsEqual(FDataRegistryEditorToolkit::RowNumberColumnId))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DataTableEditor.CellText")
				.Text(FText::FromString(FString::FromInt(RowDataPtr->RowNum)))
				.ColorAndOpacity(DataRegistryEdit, &FDataRegistryEditorToolkit::GetRowTextColor, RowDataPtr->RowId)
				.HighlightText(DataRegistryEdit, &FDataRegistryEditorToolkit::GetFilterText)
			];
	}

	if (InColumnId.IsEqual(FDataRegistryEditorToolkit::RowNameColumnId))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.Text(RowDataPtr->DisplayName)
				.HighlightText(DataRegistryEdit, &FDataRegistryEditorToolkit::GetFilterText)
				.ColorAndOpacity(DataRegistryEdit, &FDataRegistryEditorToolkit::GetRowTextColor, RowDataPtr->RowId)
			];
	}

	for (; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];
		if (ColumnData->ColumnId == InColumnId)
		{
			break;
		}
	}
	 
	// Valid column ID?
	if (AvailableColumns.IsValidIndex(ColumnIndex) && RowDataPtr->CellData.IsValidIndex(ColumnIndex))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DataTableEditor.CellText")
				.ColorAndOpacity(DataRegistryEdit, &FDataRegistryEditorToolkit::GetRowTextColor, RowDataPtr->RowId)
				.Text(DataRegistryEdit, &FDataRegistryEditorToolkit::GetCellText, RowDataPtr, ColumnIndex)
				.HighlightText(DataRegistryEdit, &FDataRegistryEditorToolkit::GetFilterText)
				.ToolTipText(DataRegistryEdit, &FDataRegistryEditorToolkit::GetCellToolTipText, RowDataPtr, ColumnIndex)
			];
	}

	return SNullWidget::NullWidget;
}

FName SDataRegistryListViewRow::GetCurrentName() const
{
	return CurrentName.IsValid() ? *CurrentName : NAME_None;

}

uint32 SDataRegistryListViewRow::GetCurrentIndex() const
{
	return RowDataPtr.IsValid() ? RowDataPtr->RowNum : -1;
}

const FDataTableEditorRowListViewDataPtr& SDataRegistryListViewRow::GetRowDataPtr() const
{
	return RowDataPtr;
}

FText SDataRegistryListViewRow::GetCurrentNameAsText() const
{
	return FText::FromName(GetCurrentName());
}

const FSlateBrush* SDataRegistryListViewRow::GetBorder() const
{
	return STableRow::GetBorder();
}

TSharedRef<SWidget> SDataRegistryListViewRow::MakeRowActionsMenu()
{
	FMenuBuilder MenuBuilder(true, DataRegistryEditor.Pin()->GetToolkitCommands());

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("FDataRegistryRowUtils", "FDataRegistryRowUtils_SearchForReferences", "Find Row References"),
		NSLOCTEXT("FDataRegistryRowUtils", "FDataRegistryRowUtils_SearchForReferencesTooltip", "Find assets that reference this Row"),
		FSlateIcon(), 
		FUIAction(FExecuteAction::CreateSP(this, &SDataRegistryListViewRow::OnSearchForReferences))
	);

	const FDataRegistrySourceItemId* FoundSource = FindSource();
	const UDataRegistrySource* SourceObject = FoundSource ? FoundSource->CachedSource.Get() : nullptr;
	if (SourceObject && !SourceObject->GetSourceAssetPath().IsNull())
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("FDataRegistryRowUtils", "FDataRegistryRowUtils_OpenSourceAsset", "Open Source Asset"),
			NSLOCTEXT("FDataRegistryRowUtils", "FDataRegistryRowUtils_OpenSourceAssetTooltip", "Open the asset editor for the source asset of this Row"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SDataRegistryListViewRow::OnOpenSourceAsset))
		);
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
