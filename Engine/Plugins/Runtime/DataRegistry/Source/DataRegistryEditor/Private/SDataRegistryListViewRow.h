// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "DataTableEditorUtils.h"

class STableViewBase;

class FDataRegistryEditorToolkit;
class SDataRegistryListViewRow;
struct FDataRegistrySourceItemId;

/**
 * A widget to represent a row in a Data Table Editor widget. This widget allows us to do things like right-click
 * and take actions on a particular row of a Data Table.
 */
class SDataRegistryListViewRow : public SMultiColumnTableRow<FDataTableEditorRowListViewDataPtr>
{
public:

	SLATE_BEGIN_ARGS(SDataRegistryListViewRow) {}
	/** The owning object. This allows us access to the actual data table being edited as well as some other API functions. */
	SLATE_ARGUMENT(TSharedPtr<FDataRegistryEditorToolkit>, DataRegistryEditor)
		/** The row we're working with to allow us to get naming information. */
		SLATE_ARGUMENT(FDataTableEditorRowListViewDataPtr, RowDataPtr)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;


	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetCurrentNameAsText() const;
	FName GetCurrentName() const;
	uint32 GetCurrentIndex() const;
	
	const FDataTableEditorRowListViewDataPtr& GetRowDataPtr() const;

private:

	const FDataRegistrySourceItemId* FindSource() const;
	void OnSearchForReferences();
	void OnOpenSourceAsset();

	TSharedRef<SWidget> MakeCellWidget(const int32 InRowIndex, const FName& InColumnId);

	virtual const FSlateBrush* GetBorder() const;

	TSharedRef<SWidget> MakeRowActionsMenu();


	TSharedPtr<FName> CurrentName;

	FDataTableEditorRowListViewDataPtr RowDataPtr;
	TWeakPtr<FDataRegistryEditorToolkit> DataRegistryEditor;

};
