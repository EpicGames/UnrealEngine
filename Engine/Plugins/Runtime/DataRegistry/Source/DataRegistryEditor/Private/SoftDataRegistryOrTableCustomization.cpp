// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftDataRegistryOrTableCustomization.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "DataRegistryEditorModule.h"
#include "IDetailChildrenBuilder.h"
#include "DataRegistryIdCustomization.h"
#include "DataRegistrySubsystem.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SoftDataRegistryOrTable"

// Makes a new instance of this detail layout class for a specific detail view requesting it
TSharedRef<IPropertyTypeCustomization> FSoftDataRegistryOrTableCustomization::MakeInstance()
{
	return MakeShareable(new FSoftDataRegistryOrTableCustomization);

}

void FSoftDataRegistryOrTableCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;
	
	FilterStructName = NAME_None;
	RegistryFilterStructName = NAME_None;

	if (StructPropertyHandle->HasMetaData(FDataRegistryType::ItemStructMetaData))
	{
		const FString& RowType = StructPropertyHandle->GetMetaData(FDataRegistryType::ItemStructMetaData);
		FilterStructName = FName(*RowType);

		// The registry filter needs the short name, but the data table is very slow unless the long name is used, so just extract the suffix
		int32 ExtIndex = -1;
		if (RowType.FindLastChar('.', ExtIndex))
		{
			RegistryFilterStructName = FName(RowType.Mid(ExtIndex+1));
		}
		else
		{
			RegistryFilterStructName = FilterStructName;
		}
	}
}

void FSoftDataRegistryOrTableCustomization::GetDataSourceComboStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	OutComboBoxStrings.Add(MakeShared<FString>(LOCTEXT("DataTable", "Data Table").ToString()));
	OutComboBoxStrings.Add(MakeShared<FString>(LOCTEXT("DataRegistry", "Data Registry").ToString()));
	OutRestrictedItems.Add(false);
	OutRestrictedItems.Add(false);	
}

bool FSoftDataRegistryOrTableCustomization::ShouldFilterDataTableAsset(const struct FAssetData& AssetData)
{
	if (!FilterStructName.IsNone())
	{
		static const FName RowStructureTagName("RowStructure");
		UScriptStruct* RowFilterStruct = UClass::TryFindTypeSlow<UScriptStruct>(FilterStructName.ToString());
		FString RowStructure;
		if (AssetData.GetTagValue<FString>(RowStructureTagName, RowStructure))
		{
			// This is slow, but at the moment we don't have an alternative to the short struct name search
			UScriptStruct* RowStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowStructure);

			if (RowStruct && RowFilterStruct && RowStruct->IsChildOf(RowFilterStruct))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

FString FSoftDataRegistryOrTableCustomization::GetDataSourceTypeValueString() const
{
	if (!UseRegistryPropertyHandle.IsValid() || !UseRegistryPropertyHandle->IsValidHandle())
	{
		return LOCTEXT("NoneValue", "None").ToString();	
	}

	bool bUseDataRegistryId = false;
	const FPropertyAccess::Result SourceResult = UseRegistryPropertyHandle->GetValue(bUseDataRegistryId);
	if (SourceResult == FPropertyAccess::Success)
	{
		if (bUseDataRegistryId)
		{
			return LOCTEXT("DataRegistry", "Data Registry").ToString();
		}
		else
		{
			return LOCTEXT("DataTable", "Data Table").ToString();	
		}
	}
	else if (SourceResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("DataTable_None", "None").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}

void FSoftDataRegistryOrTableCustomization::OnDataSourceSelected(const FString& String)
{
	const bool bUsingDataRegistry = String == LOCTEXT("DataRegistry", "Data Registry").ToString();

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (CheckHandle(UseRegistryPropertyHandle))
	{
		UseRegistryPropertyHandle->SetValue(bUsingDataRegistry);
	}

	if (bUsingDataRegistry)
	{
		// clear the data table values to prevent hidden references to a resource
		if (CheckHandle(TablePropertyHandle))
		{
			TablePropertyHandle->SetValue(static_cast<UDataTable*>(nullptr));
		}
	}
	else
	{
		// clear the data registry values to prevent hidden references to the registry
		if (CheckHandle(RegistryTypeNamePropertyHandle))
		{
			RegistryTypeNamePropertyHandle->SetValue(FName{});
		}
	}
}

FSoftDataRegistryOrTable FSoftDataRegistryOrTableCustomization::GetCurrentValue() const
{
	FSoftDataRegistryOrTable Result;

	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	if (!CheckHandle(UseRegistryPropertyHandle) ||
		!CheckHandle(RegistryTypeNamePropertyHandle) ||
		!CheckHandle(TablePropertyHandle))
	{
		return Result;
	}

	UseRegistryPropertyHandle->GetValue(Result.bUseDataRegistry);

	if (Result.bUseDataRegistry)
	{
		RegistryTypeNamePropertyHandle->GetValue(Result.RegistryType);
	}
	else
	{
		UObject* SourceDataTable = nullptr;

		TablePropertyHandle->GetValue(SourceDataTable);
		Result.Table = Cast<UDataTable>(SourceDataTable);
	}

	return Result;	
}

FString FSoftDataRegistryOrTableCustomization::OnGetDataRegistryTypeValueString() const
{
	const FSoftDataRegistryOrTable CurrentValue = GetCurrentValue();
	if (CurrentValue.bUseDataRegistry)
	{
		return CurrentValue.RegistryType.ToString();
	}

	return LOCTEXT("NoneValue", "None").ToString();
}

EVisibility FSoftDataRegistryOrTableCustomization::GetOpenDataRegistryAssetVisibility() const
{
	const FSoftDataRegistryOrTable CurrentValue = GetCurrentValue();
	return (CurrentValue.bUseDataRegistry && CurrentValue.RegistryType.IsValid()) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FSoftDataRegistryOrTableCustomization::OnClickOpenDataRegistryAsset()
{
	const FSoftDataRegistryOrTable CurrentValue = GetCurrentValue();

	if (ensure(CurrentValue.bUseDataRegistry))
	{
		if (const UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get())
		{
			if (UDataRegistry* DataRegistry = DataRegistrySubsystem->GetRegistryForType(CurrentValue.RegistryType))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DataRegistry);
			}
		}
	}

	return FReply::Handled();
}


FText FSoftDataRegistryOrTableCustomization::GetOpenDataRegistryAssetTooltip() const
{
	const FSoftDataRegistryOrTable CurrentValue = GetCurrentValue();

	if (CurrentValue.bUseDataRegistry)
	{
		if (const UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get())
		{
			if (UDataRegistry* DataRegistry = DataRegistrySubsystem->GetRegistryForType(CurrentValue.RegistryType))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Asset"), FText::AsCultureInvariant(DataRegistry->GetName()));
				return FText::Format(LOCTEXT("OpenSpecificDataRegistry", "Open '{Asset}' in the editor"), Args);
			}
		}
	}

	return LOCTEXT("OpenDataRegistry", "Open the Data Registry in the editor");
}


void FSoftDataRegistryOrTableCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	static auto CheckHandle = [](const TSharedPtr<IPropertyHandle>& Handle)
	{
		return Handle.IsValid() && Handle->IsValidHandle();
	};

	// get the child property handles
	UseRegistryPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoftDataRegistryOrTable, bUseDataRegistry));
	if (!CheckHandle(UseRegistryPropertyHandle))
	{
		return;
	}

	TSharedPtr<IPropertyHandle> RegistryTypePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoftDataRegistryOrTable, RegistryType));
	if (!CheckHandle(RegistryTypePropertyHandle))
	{
		return;
	}

	RegistryTypeNamePropertyHandle = RegistryTypePropertyHandle->GetChildHandle(FName("Name"));
	if (!CheckHandle(RegistryTypeNamePropertyHandle))
	{
		return;
	}
	
	TablePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSoftDataRegistryOrTable, Table));
	if (!CheckHandle(TablePropertyHandle))
	{
		return;
	}

	FPropertyComboBoxArgs SelectSourceTypeArgs(UseRegistryPropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateSP(this, &FSoftDataRegistryOrTableCustomization::GetDataSourceComboStrings),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FSoftDataRegistryOrTableCustomization::GetDataSourceTypeValueString),
		FOnPropertyComboBoxValueSelected::CreateSP(this, &FSoftDataRegistryOrTableCustomization::OnDataSourceSelected)
	);

	FPropertyComboBoxArgs DataRegistryIdTypeArgs(RegistryTypeNamePropertyHandle,
		FOnGetPropertyComboBoxStrings::CreateStatic(FDataRegistryEditorModule::GenerateDataRegistryTypeComboBoxStrings, true, RegistryFilterStructName),
		FOnGetPropertyComboBoxValue::CreateSP(this, &FSoftDataRegistryOrTableCustomization::OnGetDataRegistryTypeValueString));
	DataRegistryIdTypeArgs.ShowSearchForItemCount = 1;

	auto IsUsingDataRegistry = [PropertyHandleWP = UseRegistryPropertyHandle.ToWeakPtr()]()
	{
		bool bResult = false;

		TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWP.Pin();

		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			PropertyHandle->GetValue(bResult);
		}
		return bResult ? EVisibility::Visible : EVisibility::Collapsed;
	};

	auto IsUsingDataTable = [PropertyHandleWP = UseRegistryPropertyHandle.ToWeakPtr()]()
	{
		bool bResult = false;
		TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWP.Pin();

		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			PropertyHandle->GetValue(bResult);
		}
		return bResult ? EVisibility::Collapsed : EVisibility::Visible;
	};


	FUIAction CopyAction;
	FUIAction PasteAction;
	InStructPropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

	StructBuilder.AddCustomRow(LOCTEXT("DataTableOrRegistry", "Data Table Or Registry"))
		.CopyAction(CopyAction)
		.PasteAction(PasteAction)
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(800.f)
		[
			SNew(SWrapBox)
			.UseAllottedSize(true)
			.Orientation(Orient_Horizontal)
			.HAlign(HAlign_Fill)			
			+ SWrapBox::Slot()
				.FillEmptySpace(false)
				[
					PropertyCustomizationHelpers::MakePropertyComboBox(SelectSourceTypeArgs)
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SObjectPropertyEntryBox)
					.PropertyHandle(TablePropertyHandle)
					.AllowedClass(UDataTable::StaticClass())
					.OnShouldFilterAsset(this, &FSoftDataRegistryOrTableCustomization::ShouldFilterDataTableAsset)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataTable))
				]
			+ SWrapBox::Slot()
				.FillEmptySpace(true)
				[
					SNew(SHorizontalBox)
					.Visibility(TAttribute<EVisibility>::Create(IsUsingDataRegistry))
					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							PropertyCustomizationHelpers::MakePropertyComboBox(DataRegistryIdTypeArgs)
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(1, 0)
						[
							SNew(SBox)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.WidthOverride(22.0f)
							.HeightOverride(22.0f)
							.ToolTipText(this, &FSoftDataRegistryOrTableCustomization::GetOpenDataRegistryAssetTooltip)
							[
								SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "SimpleButton")
								.OnClicked(this, &FSoftDataRegistryOrTableCustomization::OnClickOpenDataRegistryAsset)
								.ContentPadding(0.0f)
								.IsFocusable(false)
								.Visibility(this, &FSoftDataRegistryOrTableCustomization::GetOpenDataRegistryAssetVisibility)
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush("SystemWideCommands.SummonOpenAssetDialog"))
										.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
				]			
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Create(
			TAttribute<bool>::CreateLambda([InStructPropertyHandle]() -> bool
			{
				return InStructPropertyHandle->IsValidHandle() && InStructPropertyHandle->DiffersFromDefault();
			}),
			FSimpleDelegate::CreateLambda([InStructPropertyHandle]()
			{
				if (InStructPropertyHandle->IsValidHandle())
				{
					InStructPropertyHandle->ResetToDefault();
				}
			})
		)
	);	
}

#undef LOCTEXT_NAMESPACE
