// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoftDataRegistryOrTable.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SToolTip.h"

/**
* Customizes a FSoftDataRegistryOrTable to support filtering by item struct, and compacting it to one row 
  */
class FSoftDataRegistryOrTableCustomization : public IPropertyTypeCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void GetDataSourceComboStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;
	FString GetDataSourceTypeValueString() const;
	void OnDataSourceSelected(const FString& String);
	FString OnGetDataRegistryTypeValueString() const;
	bool ShouldFilterDataTableAsset(const struct FAssetData& AssetData);

	EVisibility GetOpenDataRegistryAssetVisibility() const;
	FReply OnClickOpenDataRegistryAsset();
	FText GetOpenDataRegistryAssetTooltip() const;

	FSoftDataRegistryOrTable GetCurrentValue() const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> UseRegistryPropertyHandle;
	TSharedPtr<IPropertyHandle> RegistryTypeNamePropertyHandle;
	TSharedPtr<IPropertyHandle> TablePropertyHandle;

	FName FilterStructName;	
	FName RegistryFilterStructName;	
};
