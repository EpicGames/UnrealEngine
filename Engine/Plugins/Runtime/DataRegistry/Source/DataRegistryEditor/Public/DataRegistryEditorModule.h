// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API DATAREGISTRYEDITOR_API

class SToolTip;
struct FDataRegistryId;
struct FDataRegistryType;

DECLARE_DELEGATE_RetVal(FText, FOnGetDataRegistryDisplayText);
DECLARE_DELEGATE_OneParam(FOnSetDataRegistryType, FDataRegistryType);
DECLARE_DELEGATE_RetVal(FDataRegistryId, FOnGetDataRegistryId)
DECLARE_DELEGATE_OneParam(FOnSetDataRegistryId, FDataRegistryId);
DECLARE_DELEGATE_OneParam(FOnGetCustomDataRegistryItemNames, TArray<FName>&);

/**
 * The public interface to this module
 */
class FDataRegistryEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FDataRegistryEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FDataRegistryEditorModule >("DataRegistryEditor");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "DataRegistryEditor" );
	}

	/** 
	 * Creates a simple version of a Data Registry Type selector, not bound to a PropertyHandle 
	 * 
	 * @param OnGetDisplayText Delegate that returns the text to display in body of combo box
	 * @param OnSetType Delegate called when type is changed
	 * @param bAllowClear If true, add None option to top
	 * @param FilterStructName If not empty, restrict to types that use a struct that inherts from something named this
	 */
	static UE_API TSharedRef<SWidget> MakeDataRegistryTypeSelector(FOnGetDataRegistryDisplayText OnGetDisplayText, FOnSetDataRegistryType OnSetType, bool bAllowClear = true, FName FilterStructName = NAME_None);

	/** Called to get list of valid data registry types, used by MakeDataRegistryTypeSelector or can be used for a property handle version */
	static UE_API void GenerateDataRegistryTypeComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems, bool bAllowClear, FName FilterStructName);

	/**
	 * Creates a simple version of a Data Registry Item Name selector, not bound to a PropertyHandle.
	 * This will spawn the appropriate tag/combo box UI on click depending on the type
	 *
	 * @param OnGetDisplayText Delegate that returns the text to display in body of combo box
	 * @param OnGetId Delegate that should return the current id, used to populate the comobo box on click. Type must be set but name can be none
	 * @param OnSetId Delegate called when item is selected
	 * @param OnGetCustomItemNames Optional delegate to call to get custom names if OnGetId returns CustomContextType
	 * @param bAllowClear If true, add None option to top
	 */
	static UE_API TSharedRef<SWidget> MakeDataRegistryItemNameSelector(FOnGetDataRegistryDisplayText OnGetDisplayText, FOnGetDataRegistryId OnGetId, FOnSetDataRegistryId OnSetId, FOnGetCustomDataRegistryItemNames OnGetCustomItemNames = FOnGetCustomDataRegistryItemNames(), bool bAllowClear = true);

	/** Gets the extensibility managers for outside entities to extend data registry editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<class FDataRegistryGraphPanelPinFactory> DataRegistryGraphPanelPinFactory;
};

#undef UE_API
